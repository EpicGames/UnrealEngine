// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class ChaosSpatialPartitionsTests : TestModuleRules
{
	static ChaosSpatialPartitionsTests()
	{
		if (InTestMode)
		{
			TestMetadata = new Metadata();
			TestMetadata.TestName = "ChaosSpatialPartitionsTests";
			TestMetadata.TestShortName = "ChaosSpatialPartitionsTests";
			TestMetadata.ReportType = "xml";
			TestMetadata.SupportedPlatforms.Add(UnrealTargetPlatform.Linux);
			TestMetadata.SupportedPlatforms.Add(UnrealTargetPlatform.Mac);
			TestMetadata.SupportedPlatforms.Add(UnrealTargetPlatform.Android);
			TestMetadata.SupportedPlatforms.Add(UnrealTargetPlatform.IOS);

			string PlatformCompilationArgs;
			foreach (var Platform in UnrealTargetPlatform.GetValidPlatforms())
			{
				if (Platform == UnrealTargetPlatform.Android)
				{
					PlatformCompilationArgs = "-allmodules -architectures=arm64";
				}
				else
				{
					PlatformCompilationArgs = "-allmodules";
				}
				TestMetadata.PlatformCompilationExtraArgs.Add(Platform, PlatformCompilationArgs);
			}

			// Platform-specific tags
			TestMetadata.PlatformTags.Add(UnrealTargetPlatform.Linux, "~[.]~[Slow]");
			TestMetadata.PlatformTags.Add(UnrealTargetPlatform.Android, "~[Perf]~[Slow]~[AndroidSkip]");

			// Allow Android run for this test
			// Will remove Android from PlatformsRunUnsupported as more diverse types of tests can run on this platform
			TestMetadata.PlatformsRunUnsupported.Remove(UnrealTargetPlatform.Android);
		}
	}

	public ChaosSpatialPartitionsTests(ReadOnlyTargetRules Target) : base(Target)
	{
		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"Core",
				"CoreUObject",
				"ChaosCore",
				"ChaosSpatialPartitions",
				"ChaosTestHarness",
				"Engine",
		});

		bTreatAsEngineModule = true;
		bWarningsAsErrors = true;
	}
}
