// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

[SupportedPlatforms(UnrealPlatformClass.All)]
 public class ChaosSpatialPartitionsTestsTarget : ChaosTestHarnessTarget
 {
     public ChaosSpatialPartitionsTestsTarget(TargetInfo Target) : base(Target)
     {
		TestTargetRules.bTestsRequireEngine = true;
        bWithLowLevelTestsOverride = true;
     }
 }
 