// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "ChaosTestHarness.h"
#include "catch2/matchers/catch_matchers_floating_point.hpp"

#include "AabbTreeTestHelpers.h"
#include "StaticAabbTreeHelpers.h"

#include "ChaosSpatialPartitions/Algorithms/StaticAabbTreeTimeSlicer.h"
#include "ObjectData.h"

namespace Chaos::SpatialPartition::AabbTreeAlgorithm
{
	using FTimeSlicerConfig = FStaticAabbTreeTimeSlicer::FConfig;

	TArray<FAabbTreeLeaf> RunAndGetLeaves(const FTimeSlicerConfig& Config, TArray<FAabbTreeLeafElement>& Elements)
	{
		FStaticAabbTreeTimeSlicer TimeSlicer(Config, MoveTemp(Elements));
		while (TimeSlicer.Run())
		{
		}
		return TimeSlicer.GetLeaves();
	};

	TEST_CASE("StaticAabbTreeTimeSlicer - MaxElementsPerLeaf", "[Chaos][StaticAabbTreeTimeSlicer][spatial-partition]")
	{
		FTimeSlicerConfig Config;
		Config.BatchSize = 10000;
		Config.PartitioningMethod = GENERATE(EPartitioningMethod::CentroidSpatialMedian, EPartitioningMethod::CentroidVariance, EPartitioningMethod::SurfaceArea);
		TArray<FAabbTreeLeafElement> Elements;
		for (int32 I = 0; I < 10; ++I)
		{
			Elements.Emplace(FAabbTreeLeafElement{ .Aabb = FAABB3(FVec3(I, 0, 0), FVec3(I + 1, 1, 1)), .Index = I });
		}

		SECTION("ElementsPerLeaf: 10")
		{
			Config.MaxElementsPerLeaf = 10;
			const TArray<FAabbTreeLeaf> Leaves = RunAndGetLeaves(Config, Elements);

			CHECK(Leaves.Num() == 1);
			CHECK(Leaves[0].Elements.Num() == Config.MaxElementsPerLeaf);
		}
		SECTION("ElementsPerLeaf: 5")
		{
			Config.MaxElementsPerLeaf = 5;
			const TArray<FAabbTreeLeaf> Leaves = RunAndGetLeaves(Config, Elements);

			int32 ActualElementCount = 0;
			for (const FAabbTreeLeaf& Leaf : Leaves)
			{
				CHECK(Leaf.Elements.Num() <= Config.MaxElementsPerLeaf);
				ActualElementCount += Leaf.Elements.Num();
			}
			CHECK(ActualElementCount == 10);
		}
		SECTION("ElementsPerLeaf: 2")
		{
			Config.MaxElementsPerLeaf = 2;
			const TArray<FAabbTreeLeaf> Leaves = RunAndGetLeaves(Config, Elements);

			int32 ActualElementCount = 0;
			for (const FAabbTreeLeaf& Leaf : Leaves)
			{
				CHECK(Leaf.Elements.Num() <= Config.MaxElementsPerLeaf);
				ActualElementCount += Leaf.Elements.Num();
			}
			CHECK(ActualElementCount == 10);
		}
		SECTION("ElementsPerLeaf: 1")
		{
			Config.MaxElementsPerLeaf = 1;
			const TArray<FAabbTreeLeaf> Leaves = RunAndGetLeaves(Config, Elements);

			CHECK(Leaves.Num() == 10);
			for (const FAabbTreeLeaf& Leaf : Leaves)
			{
				CHECK(Leaf.Elements.Num() == Config.MaxElementsPerLeaf);
			}
		}
	}

	TEST_CASE("StaticAabbTreeTimeSlicer - MaxTreeDepth", "[Chaos][StaticAabbTreeTimeSlicer][spatial-partition]")
	{
		// Test max tree depth, which takes priority over the max elements per leaf. Set max elements per leaf to 1 to make testing easier
		FTimeSlicerConfig Config;
		Config.BatchSize = 10000;
		Config.MaxElementsPerLeaf = 1;
		Config.PartitioningMethod = GENERATE(EPartitioningMethod::CentroidSpatialMedian, EPartitioningMethod::CentroidVariance, EPartitioningMethod::SurfaceArea);

		TArray<FAabbTreeLeafElement> Elements;
		for (int32 I = 0; I < 8; ++I)
		{
			Elements.Emplace(FAabbTreeLeafElement{ .Aabb = FAABB3(FVec3(I, 0, 0), FVec3(I + 1, 1, 1)), .Index = I });
		}

		SECTION("MaxTreeDepth: 0")
		{
			Config.MaxTreeDepth = 0;

			FStaticAabbTreeTimeSlicer TimeSlicer(Config, MoveTemp(Elements));
			while (TimeSlicer.Run()) {}
			const TArray<FAabbTreeNode> Nodes = TimeSlicer.GetNodes();

			CHECK(Nodes.Num() == 1);
		}
		SECTION("MaxTreeDepth: 1")
		{
			Config.MaxTreeDepth = 1;

			FStaticAabbTreeTimeSlicer TimeSlicer(Config, MoveTemp(Elements));
			while (TimeSlicer.Run()) {}
			const TArray<FAabbTreeNode> Nodes = TimeSlicer.GetNodes();

			CHECK(Nodes.Num() == 3);
		}
		SECTION("MaxTreeDepth: 5")
		{
			Config.MaxTreeDepth = 5;

			FStaticAabbTreeTimeSlicer TimeSlicer(Config, MoveTemp(Elements));
			while (TimeSlicer.Run()) {}
			const TArray<FAabbTreeNode> Nodes = TimeSlicer.GetNodes();

			// 8 + 4 + 2 + 1 = 15
			CHECK(Nodes.Num() == 15);
		}
	}

	TEST_CASE("StaticAabbTreeTimeSlicer - Basic", "[Chaos][StaticAabbTreeTimeSlicer][spatial-partition]")
	{
		const FVec3 AabbSize(1);

		FTimeSlicerConfig Config;
		Config.MaxTreeDepth = 1000;
		Config.MaxElementsPerLeaf = 1;
		Config.PartitioningMethod = GENERATE(EPartitioningMethod::CentroidSpatialMedian, EPartitioningMethod::CentroidVariance, EPartitioningMethod::SurfaceArea);
		Config.BatchSize = GENERATE(1, 10000);

		// Basic data set where it's better to split on Y then X then Z. This is true for all 3 partitioning methods
		TArray<FAABB3> Aabbs
		{
			BuildAabbMinExtents(FVec3(0, 0, 0), AabbSize),
			BuildAabbMinExtents(FVec3(8, 0, 0), AabbSize),
			BuildAabbMinExtents(FVec3(0, 10, 0), AabbSize),
			BuildAabbMinExtents(FVec3(8, 10, 0), AabbSize),
			BuildAabbMinExtents(FVec3(0, 0, 4), AabbSize),
			BuildAabbMinExtents(FVec3(8, 0, 4), AabbSize),
			BuildAabbMinExtents(FVec3(0, 10, 4), AabbSize),
			BuildAabbMinExtents(FVec3(8, 10, 4), AabbSize),
		};
		// Split0: [0, 1, 4, 5], [2, 3, 6, 7]
		// Split1: ([0, 4], [1, 5]), ([2, 6], [3, 7])
		//           A
		//     B           C
		//  D     E      F     G
		// 0 4   1 5    2 6   3 7

		TArray<FAabbTreeLeafElement> Elements = BuildElementsFromAabbs(Aabbs);
		FStaticAabbTreeTimeSlicer TimeSlicer(Config, MoveTemp(Elements));
		while (TimeSlicer.Run())
		{
		}
		const int32 RootIndex = TimeSlicer.GetRootIndex();
		const TArray<FAabbTreeNode> Nodes = TimeSlicer.GetNodes();
		const TArray<FAabbTreeLeaf> Leaves = TimeSlicer.GetLeaves();

		// Find what leaf/node each user data ended up at
		TArray<const FAabbTreeLeaf*> ElementLeaves;
		TArray<const FAabbTreeNode*> ElementNodes;
		FindLeafAndNodePointers(Leaves, Nodes, Aabbs.Num(), ElementLeaves, ElementNodes);

		// Check the leaves and their nodes
		for (int32 I = 1; I < Aabbs.Num(); ++I)
		{
			CHECK(ElementLeaves[I]->Elements.Num() == 1);
			CHECK_THAT(ElementLeaves[I]->Aabb, Catch::Equal(Aabbs[I]));
			CHECK_THAT(ElementNodes[I]->Aabb, Catch::Equal(Aabbs[I]));
		}

		const FAABB3 Aabb04 = Union(Aabbs[0], Aabbs[4]);
		const FAABB3 Aabb15 = Union(Aabbs[1], Aabbs[5]);
		const FAABB3 Aabb26 = Union(Aabbs[2], Aabbs[6]);
		const FAABB3 Aabb37 = Union(Aabbs[3], Aabbs[7]);
		const FAABB3 Aabb0415 = Union(Aabb04, Aabb15);
		const FAABB3 Aabb2637 = Union(Aabb26, Aabb37);
		const FAABB3 RootAabb = Union(Aabb0415, Aabb2637);

		// Height 1:
		// Node(0,4)
		CHECK(ElementNodes[0]->Parent == ElementNodes[4]->Parent);
		const FAabbTreeNode& Node04 = Nodes[ElementNodes[0]->Parent];
		CHECK_THAT(Node04.Aabb, Catch::Equal(Aabb04));
		// Node(1,5)
		CHECK(ElementNodes[1]->Parent == ElementNodes[5]->Parent);
		const FAabbTreeNode& Node15 = Nodes[ElementNodes[1]->Parent];
		CHECK_THAT(Node15.Aabb, Catch::Equal(Aabb15));
		// Node(2,6)
		CHECK(ElementNodes[2]->Parent == ElementNodes[6]->Parent);
		const FAabbTreeNode& Node26 = Nodes[ElementNodes[2]->Parent];
		CHECK_THAT(Node26.Aabb, Catch::Equal(Aabb26));
		// Node(3,7)
		CHECK(ElementNodes[3]->Parent == ElementNodes[7]->Parent);
		const FAabbTreeNode& Node37 = Nodes[ElementNodes[3]->Parent];
		CHECK_THAT(Node37.Aabb, Catch::Equal(Aabb37));

		// Height 2:
		// Node(04,15)
		CHECK(Node04.Parent == Node15.Parent);
		const FAabbTreeNode& Node04_15 = Nodes[Node04.Parent];
		CHECK_THAT(Node04_15.Aabb, Catch::Equal(Aabb0415));
		// Node(26,37)
		CHECK(Node26.Parent == Node37.Parent);
		const FAabbTreeNode& Node26_37 = Nodes[Node26.Parent];
		CHECK_THAT(Node26_37.Aabb, Catch::Equal(Aabb2637));

		// Height 3 (Root):
		CHECK(Node04_15.Parent == Node26_37.Parent);
		CHECK(Node04_15.Parent == RootIndex);
		CHECK_THAT(Nodes[RootIndex].Aabb, Catch::Equal(RootAabb));
	}

	TEST_CASE("StaticAabbTreeTimeSlicer - MedianVariance Heuristic", "[Chaos][StaticAabbTreeTimeSlicer][spatial-partition]")
	{
		const FVec3 AabbSize(1);

		FTimeSlicerConfig Config;
		Config.MaxTreeDepth = 1000;
		Config.MaxElementsPerLeaf = 1;
		Config.PartitioningMethod = EPartitioningMethod::CentroidVariance;
		Config.BatchSize = 10000;

		TArray<FAABB3> Aabbs
		{
			BuildAabbCenterExtents(FVec3(0, 3, 0), AabbSize),
			BuildAabbCenterExtents(FVec3(3, 0, 0), AabbSize),
			BuildAabbCenterExtents(FVec3(4, 0, 0), AabbSize),
			BuildAabbCenterExtents(FVec3(4, 4, 0), AabbSize),
			BuildAabbCenterExtents(FVec3(6, 3, 0), AabbSize),
			BuildAabbCenterExtents(FVec3(7, 7, 0), AabbSize),
		};
		//7|              5
		//6|               
		//5|               
		//4|        3      
		//3|0           4       
		//2|               
		//1|               
		//0|      1 2
		//  ---------------
		//  0 1 2 3 4 5 6 7
		// Split0: (12), (0345) (Y axis with median 2.8)
		// Split1: (12), ((03),(45)) (X axis with median 4.25)
		// Note: For each split, there are objects on the other side of the spatial median
		// that are actually dragged to the other side because of using the mean.
		// For example, for the first split the median is 3.5 but the mean is 2.8, hence
		// we end up with (12), (0345) instead of (0124), (35).
		//       A
		//   B       C
		//  1 2   D     E
		//       0 3   4 5

		TArray<FAabbTreeLeafElement> Elements = BuildElementsFromAabbs(Aabbs);
		FStaticAabbTreeTimeSlicer TimeSlicer(Config, MoveTemp(Elements));
		while (TimeSlicer.Run())
		{
		}
		const int32 RootIndex = TimeSlicer.GetRootIndex();
		const TArray<FAabbTreeNode> Nodes = TimeSlicer.GetNodes();
		const TArray<FAabbTreeLeaf> Leaves = TimeSlicer.GetLeaves();

		TArray<const FAabbTreeLeaf*> ElementLeaves;
		TArray<const FAabbTreeNode*> ElementNodes;
		FindLeafAndNodePointers(Leaves, Nodes, Aabbs.Num(), ElementLeaves, ElementNodes);

		// Check the leaves and their nodes
		for (int32 I = 1; I < Aabbs.Num(); ++I)
		{
			CHECK(ElementLeaves[I]->Elements.Num() == 1);
			CHECK_THAT(ElementLeaves[I]->Aabb, Catch::Equal(Aabbs[I]));
			CHECK_THAT(ElementNodes[I]->Aabb, Catch::Equal(Aabbs[I]));
		}

		const FAABB3 Aabb12 = Union(Aabbs[1], Aabbs[2]);
		const FAABB3 Aabb03 = Union(Aabbs[0], Aabbs[3]);
		const FAABB3 Aabb45 = Union(Aabbs[4], Aabbs[5]);
		const FAABB3 Aabb0345 = Union(Aabb03, Aabb45);
		const FAABB3 RootAabb = Union(Aabb12, Aabb0345);

		// Height 1:
		// Node(1,2)
		CHECK(ElementNodes[1]->Parent == ElementNodes[2]->Parent);
		const FAabbTreeNode& Node12 = Nodes[ElementNodes[1]->Parent];
		CHECK_THAT(Node12.Aabb, Catch::Equal(Aabb12));
		// Node(0,3)
		CHECK(ElementNodes[0]->Parent == ElementNodes[3]->Parent);
		const FAabbTreeNode& Node03 = Nodes[ElementNodes[0]->Parent];
		CHECK_THAT(Node03.Aabb, Catch::Equal(Aabb03));
		// Node(4,5)
		CHECK(ElementNodes[4]->Parent == ElementNodes[5]->Parent);
		const FAabbTreeNode& Node45 = Nodes[ElementNodes[4]->Parent];
		CHECK_THAT(Node45.Aabb, Catch::Equal(Aabb45));

		// Height 2:
		// Node (03,45)
		CHECK(Node03.Parent == Node45.Parent);
		const FAabbTreeNode& Node03_45 = Nodes[Node03.Parent];
		CHECK_THAT(Node03_45.Aabb, Catch::Equal(Aabb0345));

		// Height 3 (Root):
		CHECK(Node12.Parent == Node03_45.Parent);
		CHECK(Node12.Parent == RootIndex);
		CHECK_THAT(Nodes[RootIndex].Aabb, Catch::Equal(RootAabb));
	}

	TEST_CASE("StaticAabbTreeTimeSlicer - SurfaceArea Heuristic", "[Chaos][StaticAabbTreeTimeSlicer][spatial-partition]")
	{
		const FVec3 AabbSize(1);

		FTimeSlicerConfig Config;
		Config.MaxTreeDepth = 1000;
		Config.MaxElementsPerLeaf = 1;
		Config.PartitioningMethod = EPartitioningMethod::SurfaceArea;
		Config.BatchSize = 10000;

		// Build a set of objects roughly like:
		//           4 5
		//0
		//      1
		//       2
		//         
		//        3
		// This should first split on the X: (0, 1, 2, 3), (4, 5)
		// Then split on the Y: ((1, 2, 3), 0), (4,5)
		// Then split again on the Y: ((3, (1, 2)), 0), (4, 5)
		//          A
		//      B       C
		//    D   0    4  5
		//  3  E
		//    1 2
		TArray<FAABB3> Aabbs
		{
			BuildAabbMinExtents(FVec3(0, 8, 0), AabbSize),
			BuildAabbMinExtents(FVec3(4, 4, 0), AabbSize),
			BuildAabbMinExtents(FVec3(4, 5, 0), AabbSize),
			BuildAabbMinExtents(FVec3(6, 0, 0), AabbSize),
			BuildAabbMinExtents(FVec3(10, 9, 0), AabbSize),
			BuildAabbMinExtents(FVec3(11, 9, 0), AabbSize),
		};

		TArray<FAabbTreeLeafElement> Elements = BuildElementsFromAabbs(Aabbs);
		FStaticAabbTreeTimeSlicer TimeSlicer(Config, MoveTemp(Elements));
		while (TimeSlicer.Run())
		{
		}
		const int32 RootIndex = TimeSlicer.GetRootIndex();
		const TArray<FAabbTreeNode> Nodes = TimeSlicer.GetNodes();
		const TArray<FAabbTreeLeaf> Leaves = TimeSlicer.GetLeaves();


		TArray<const FAabbTreeLeaf*> ElementLeaves;
		TArray<const FAabbTreeNode*> ElementNodes;
		FindLeafAndNodePointers(Leaves, Nodes, Aabbs.Num(), ElementLeaves, ElementNodes);

		// Check the leaves and their nodes
		for (int32 I = 1; I < Aabbs.Num(); ++I)
		{
			CHECK(ElementLeaves[I]->Elements.Num() == 1);
			CHECK_THAT(ElementLeaves[I]->Aabb, Catch::Equal(Aabbs[I]));
			CHECK_THAT(ElementNodes[I]->Aabb, Catch::Equal(Aabbs[I]));
		}

		const FAABB3 Aabb12 = Union(Aabbs[1], Aabbs[2]);
		const FAABB3 Aabb123 = Union(Aabb12, Aabbs[3]);
		const FAABB3 Aabb0123 = Union(Aabb123, Aabbs[0]);
		const FAABB3 Aabb45 = Union(Aabbs[4], Aabbs[5]);
		const FAABB3 RootAabb = Union(Aabb0123, Aabb45);

		// Height 1:
		// Node(1,2)
		CHECK(ElementNodes[1]->Parent == ElementNodes[2]->Parent);
		const FAabbTreeNode& Node12 = Nodes[ElementNodes[1]->Parent];
		CHECK_THAT(Node12.Aabb, Catch::Equal(Aabb12));
		// Height 2:
		// Node(3, 12)
		CHECK(ElementNodes[3]->Parent == Node12.Parent);
		const FAabbTreeNode& Node3_12 = Nodes[ElementNodes[3]->Parent];
		CHECK_THAT(Node3_12.Aabb, Catch::Equal(Aabb123));
		// Height 3:
		// Node(3_12_0)
		CHECK(Node3_12.Parent == ElementNodes[0]->Parent);
		const FAabbTreeNode& Node3_12_0 = Nodes[Node3_12.Parent];
		CHECK_THAT(Node3_12_0.Aabb, Catch::Equal(Aabb0123));
		// Node(4,5)
		CHECK(ElementNodes[4]->Parent == ElementNodes[5]->Parent);
		const FAabbTreeNode& Node45 = Nodes[ElementNodes[4]->Parent];
		CHECK_THAT(Node45.Aabb, Catch::Equal(Aabb45));
		// Height 4 (Root):
		CHECK(Node3_12_0.Parent == Node45.Parent);
		CHECK(Node3_12_0.Parent == RootIndex);
		CHECK_THAT(Nodes[RootIndex].Aabb, Catch::Equal(RootAabb));
	}

	TEST_CASE("StaticAabbTreeTimeSlicer - Identical aabbs", "[Chaos][StaticAabbTreeTimeSlicer][spatial-partition]")
	{
		// Test a scenario where all aabbs are identical. 
		// Since all of the aabbs are identical, all heuristic methods will partition every object to same size. 
		// If no special handling is performed, this will produce a list where all elements are in the node of MaxTreeDepth.
		const FVec3 AabbSize(1);
		TArray<FAabbTreeLeafElement> Elements;
		for (int32 I = 0; I < 8; ++I)
		{
			Elements.Emplace(FAabbTreeLeafElement{ .Aabb = BuildAabbMinExtents(FVec3(0, 0, 0), AabbSize), .Index = I });
		}

		FTimeSlicerConfig Config;
		Config.MaxTreeDepth = 1000;
		Config.MaxElementsPerLeaf = 1;
		Config.BatchSize = 10000;
		Config.PartitioningMethod = GENERATE(EPartitioningMethod::CentroidSpatialMedian, EPartitioningMethod::CentroidVariance, EPartitioningMethod::SurfaceArea);

		FStaticAabbTreeTimeSlicer TimeSlicer(Config, MoveTemp(Elements));
		while (TimeSlicer.Run()) {}

		const int32 RootIndex = TimeSlicer.GetRootIndex();
		const TArray<FAabbTreeNode> Nodes = TimeSlicer.GetNodes();
		const TArray<FAabbTreeLeaf> Leaves = TimeSlicer.GetLeaves();
		// 8 + 4 + 2 + 1 == 15
		CHECK(Nodes.Num() == 15);
		CHECK(Leaves.Num() == 8);
	}

	TEST_CASE("StaticAabbTreeTimeSlicer - Performance", "[Chaos][StaticAabbTreeTimeSlicer][spatial-partition][!benchmark]")
	{
		TArray<FObjectData> Objects;
		BuildObjectList(Objects, 100, 50, 5);
		TArray<FAabbTreeLeafElement> SourceElements;
		for (int32 I = 0; I < Objects.Num(); ++I)
		{
			SourceElements.Emplace(FAabbTreeLeafElement{ .Aabb = Objects[I].Aabb, .Index = I });
		}

		FTimeSlicerConfig Config;
		Config.MaxTreeDepth = 1000;
		Config.MaxElementsPerLeaf = 1;
		Config.BatchSize = 10000;

		auto BuildAndRun = [&Config, &SourceElements](EPartitioningMethod PartitioningMethod) -> int32
		{
			Config.PartitioningMethod = PartitioningMethod;
			TArray<FAabbTreeLeafElement> Elements = SourceElements;
			FStaticAabbTreeTimeSlicer TimeSlicer(Config, MoveTemp(Elements));
			while (TimeSlicer.Run()) {}
			return TimeSlicer.GetRootIndex();
		};

		BENCHMARK("CentroidSpatialMedian")
		{
			return BuildAndRun(EPartitioningMethod::CentroidSpatialMedian);
		};
		BENCHMARK("CentroidVariance")
		{
			return BuildAndRun(EPartitioningMethod::CentroidVariance);
		};
		BENCHMARK("SurfaceArea")
		{
			return BuildAndRun(EPartitioningMethod::SurfaceArea);
		};
	}
} // namespace Chaos::SpatialPartition::AabbTreeAlgorithm
