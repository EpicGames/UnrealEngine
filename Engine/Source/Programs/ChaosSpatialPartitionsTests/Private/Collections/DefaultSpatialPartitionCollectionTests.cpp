// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "ChaosTestHarness.h"

#include "TestVisitors.h"
#include "SharedOverlapTests.h"
#include "SharedRaycastTests.h"
#include "SharedSweepTests.h"
#include "SharedSelfQueryTests.h"
#include "SharedCollectionTests.h"

#include "ChaosSpatialPartitions/Collections/DefaultSpatialPartitionCollection.h"

namespace Chaos::SpatialPartition
{
	class FTestDefaultSpatialPartitionCollection : public FDefaultSpatialPartitionCollection
	{
	public:
		bool bAutoRebuild = true;

		using FDefaultSpatialPartitionCollection::FDefaultSpatialPartitionCollection;
		using FDefaultSpatialPartitionCollection::Insert;
		using FDefaultSpatialPartitionCollection::Update;
		using FDefaultSpatialPartitionCollection::Remove;

		void Insert(FObjectData& Object)
		{
			Insert(Object.UserData, Object.Aabb, Object.Classification, Object.Handle);
			TryAutoRebuild();
		}

		void Update(FObjectData& Object)
		{
			Update(Object.UserData, Object.Aabb, Object.Classification, Object.Handle);
			TryAutoRebuild();
		}

		void Remove(FObjectData& Object)
		{
			Remove(Object.Handle);
			TryAutoRebuild();
		}

		void ForceRebuild()
		{
			while (ISpatialPartition::ERebuildStatus::Continue == Rebuild())
			{
			}
		}

		void TryAutoRebuild()
		{
			if (!bAutoRebuild || !NeedsRebuilding())
			{
				return;
			}

			ForceRebuild();
		}

		static FTestDefaultSpatialPartitionCollection* CreateForPerfTest()
		{
			FTestDefaultSpatialPartitionCollection::FConfig Config;
			Config.StaticConfig.PartitioningMethod = EPartitioningMethod::CentroidVariance;
			Config.StaticConfig.BatchSize = 250;
			FTestDefaultSpatialPartitionCollection* Result = new FTestDefaultSpatialPartitionCollection(Config);
			Result->bAutoRebuild = false;
			return Result;
		};
	};

	template <>
	void InsertObject(FTestDefaultSpatialPartitionCollection& SpatialPartition, FObjectData& Object)
	{
		SpatialPartition.Insert(Object);
	}

	template <>
	void UpdateObject(FTestDefaultSpatialPartitionCollection& SpatialPartition, FObjectData& Object)
	{
		SpatialPartition.Update(Object);
	}

	template <>
	void RemoveObject(FTestDefaultSpatialPartitionCollection& SpatialPartition, FObjectData& Object)
	{
		SpatialPartition.Remove(Object);
	}

	template <>
	void BuildFromObjects(FTestDefaultSpatialPartitionCollection& SpatialPartition, TArray<FObjectData>& Objects)
	{
		const bool bAutoRebuild = SpatialPartition.bAutoRebuild;
		SpatialPartition.bAutoRebuild = false;
		for (FObjectData& Object : Objects)
		{
			SpatialPartition.Insert(Object);
		}
		SpatialPartition.bAutoRebuild = bAutoRebuild;
		SpatialPartition.TryAutoRebuild();
	}

	template <>
	void Rebuild(FTestDefaultSpatialPartitionCollection& SpatialPartition)
	{
		SpatialPartition.ForceRebuild();
	}

	TEST_CASE("DefaultSpatialPartitionCollection - Build", "[Chaos][DefaultSpatialPartitionCollection][spatial-partition]")
	{
		FTestDefaultSpatialPartitionCollection Collection;
		Collection.bAutoRebuild = GENERATE(false, true);
		TestCollectionBasicBuild(Collection);
	}

	TEST_CASE("DefaultSpatialPartitionCollection - Build Large", "[Chaos][DefaultSpatialPartitionCollection][spatial-partition]")
	{
		// Test with a large number of objects to enforce time slicing will take a while.
		constexpr int32 ObjectCount = 10000;
		FTestDefaultSpatialPartitionCollection::FConfig Config;
		FTestDefaultSpatialPartitionCollection Collection(Config);
		Collection.bAutoRebuild = true;

		const TArray<FSpatialClassification> Classifications = GetClassifications();
		TArray<FObjectData> Objects;
		BuildObjectList(Objects, ObjectCount, Classifications);
		BuildFromObjects(Collection, Objects);

		FOverlapQueryData QueryData;
		QueryData.Aabb = Objects[0].Aabb;
		QueryData.Aabb.GrowToInclude(Objects.Last().Aabb);
		FTestOverlapCollectorVisitor Visitor;
		Collection.Overlap(QueryData, Visitor);
		CHECK(Visitor.Results.Num() == Objects.Num());
	}

	TEST_CASE("DefaultSpatialPartitionCollection - Insert", "[Chaos][DefaultSpatialPartitionCollection][spatial-partition]")
	{
		FTestDefaultSpatialPartitionCollection Collection;
		Collection.bAutoRebuild = GENERATE(false, true);
		TestCollectionBasicInsert(Collection);
	}

	TEST_CASE("DefaultSpatialPartitionCollection - Update - Same Category", "[Chaos][DefaultSpatialPartitionCollection][spatial-partition]")
	{
		FTestDefaultSpatialPartitionCollection Collection;
		Collection.bAutoRebuild = GENERATE(false, true);
		TestCollectionBasicUpdateSameCategory(Collection);
	}

	TEST_CASE("DefaultSpatialPartitionCollection - Update - Different Category", "[Chaos][DefaultSpatialPartitionCollection][spatial-partition]")
	{
		FTestDefaultSpatialPartitionCollection Collection;
		Collection.bAutoRebuild = GENERATE(false, true);
		TestCollectionBasicUpdateDifferentCategory(Collection);
	}

	TEST_CASE("DefaultSpatialPartitionCollection - Remove", "[Chaos][DefaultSpatialPartitionCollection][spatial-partition]")
	{
		FTestDefaultSpatialPartitionCollection Collection;
		Collection.bAutoRebuild = GENERATE(false, true);
		TestCollectionBasicRemove(Collection);
	}

	TEST_CASE("DefaultSpatialPartitionCollection - Insert Perf", "[Chaos][DefaultSpatialPartitionCollection][spatial-partition][!benchmark]")
	{
		TestCollectionBasicInsertPerf<FTestDefaultSpatialPartitionCollection>(FTestDefaultSpatialPartitionCollection::CreateForPerfTest);
	}

	TEST_CASE("DefaultSpatialPartitionCollection - Update Perf", "[Chaos][DefaultSpatialPartitionCollection][spatial-partition][!benchmark]")
	{
		TestCollectionBasicUpdatePerf<FTestDefaultSpatialPartitionCollection>(FTestDefaultSpatialPartitionCollection::CreateForPerfTest);
	}

	TEST_CASE("DefaultSpatialPartitionCollection - Remove Perf", "[Chaos][DefaultSpatialPartitionCollection][spatial-partition][!benchmark]")
	{
		TestCollectionBasicRemovePerfWithSplitClassifications<FTestDefaultSpatialPartitionCollection>(FTestDefaultSpatialPartitionCollection::CreateForPerfTest);
	}

	TEST_CASE("DefaultSpatialPartitionCollection - Rebuild Perf", "[Chaos][DefaultSpatialPartitionCollection][spatial-partition][!benchmark]")
	{
		TestCollectionBasicRebuildPerf<FTestDefaultSpatialPartitionCollection>(FTestDefaultSpatialPartitionCollection::CreateForPerfTest);
	}

	TEST_CASE("DefaultSpatialPartitionCollection - Query Perf", "[Chaos][DefaultSpatialPartitionCollection][spatial-partition][!benchmark]")
	{
		FTestDefaultSpatialPartitionCollection Collection;
		Collection.bAutoRebuild = false;
		TestCollectionBasicOverlapPerf(Collection);
	}

	TEST_CASE("DefaultSpatialPartitionCollection - Query During Rebuild", "[Chaos][DefaultSpatialPartitionCollection][spatial-partition]")
	{
		constexpr int32 ObjectCount = 10;
		TArray<FObjectData> Objects;
		BuildObjectList(Objects, ObjectCount, GetClassifications());

		FTestDefaultSpatialPartitionCollection::FConfig Config;
		Config.StaticConfig.BatchSize = 1;
		Config.StaticConfig.TargetProcessingTimeInSeconds = 0;
		Config.StaticConfig.MaxElementsPerLeaf = 1;
		FTestDefaultSpatialPartitionCollection Collection(Config);
		Collection.bAutoRebuild = false;

		BuildFromObjects(Collection, Objects);
		Collection.ForceRebuild();

		for (FObjectData& Obj : Objects)
		{
			Obj.Aabb.MoveByVector(FVec3(0, 1, 0));
			Collection.Update(Obj);
		}

		const ISpatialPartition::ERebuildStatus RebuildStatus = Collection.Rebuild();
		CHECK(RebuildStatus == ISpatialPartition::ERebuildStatus::Continue);

		// Verify a query for each object
		for (const FObjectData& Obj : Objects)
		{
			const FVec3 Center = Obj.Aabb.Center();
			FOverlapQueryData QueryData;
			QueryData.Aabb = FAABB3(Center, Center);
			FTestOverlapCollectorVisitor Visitor;
			Collection.Overlap(QueryData, Visitor);
			CHECK(Visitor.Results == TArray<FUserDataType>{Obj.UserData});
		}
	}
} // namespace Chaos::SpatialPartition
