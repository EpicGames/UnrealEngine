// Copyright Epic Games, Inc. All Rights Reserved.

#include "Collections/LegacySpatialPartitionCollection.h"

#include "Chaos/PBDRigidsEvolution.h"

namespace Chaos::SpatialPartition
{
	FLegacySpatialPartitionCollection::FConfig::FConfig()
	{
		bStaticOnly = false;
		StaticSettings.MaxChildrenInLeaf = BroadPhaseConfig.MaxChildrenInLeaf;
		StaticSettings.MaxTreeDepth = BroadPhaseConfig.MaxTreeDepth;
		StaticSettings.MaxPayloadSize = BroadPhaseConfig.MaxPayloadSize;
		StaticSettings.IterationsPerTimeSlice = BroadPhaseConfig.IterationsPerTimeSlice;
		StaticSettings.bBuildOverlapCache = true;
		DynamicSettings = StaticSettings;
		StaticSettings.bUseDirtyTree = true;
	}

	FLegacySpatialPartitionCollection::FLegacySpatialPartitionCollection(const FConfig& InConfig)
		: Config(InConfig)
	{
		const FConfig::FSettings& SConfig = Config.StaticSettings;
		StaticAcceleration = MakeUnique<FStaticAabbTreeType>(FStaticAabbTreeType::EmptyInit(), SConfig.MaxChildrenInLeaf, SConfig.MaxTreeDepth, SConfig.MaxPayloadSize, SConfig.IterationsPerTimeSlice, false, SConfig.bUseDirtyTree, SConfig.bBuildOverlapCache);
		if (!Config.bStaticOnly)
		{
			const FConfig::FSettings& DConfig = Config.DynamicSettings;
			DynamicAcceleration = MakeUnique<FDynamicAabbTreeType>(FDynamicAabbTreeType::EmptyInit(), DConfig.MaxChildrenInLeaf, DConfig.MaxTreeDepth, DConfig.MaxPayloadSize, DConfig.IterationsPerTimeSlice, true, DConfig.bUseDirtyTree, DConfig.bBuildOverlapCache);
		}
	}

	void FLegacySpatialPartitionCollection::Insert(const FUserDataType& UserData, const FAABB3& Aabb, const FSpatialClassification& Classification, FSpatialHandle& OutHandle)
	{
		check(!bIsTimeSliceRebuilding);

		const FEntryHandle EntryHandle = Entries.Create(FEntry{ .Aabb = Aabb, .Classification = Classification, .UserData = UserData });
		// Note: We use the index for the handle value instead of AsUint() as the AabbTree uses TArrayMap which will allocate an array to contain N items. 
		// Since the handle is a mix of index/generation, it will grow to very large numbers and this will destroy perf.
		const int32 HandleValue = EntryHandle.GetIndex();

		FSpatialAcceleration* AccelerationStructure = GetAccelerationStructure(Classification);
		AccelerationStructure->UpdateElement(HandleValue, Aabb, true);

		OutHandle.SetValue(HandleValue);
	}

	void FLegacySpatialPartitionCollection::Update(const FUserDataType& UserData, const FAABB3& Aabb, const FSpatialClassification& Classification, FSpatialHandle& InOutHandle)
	{
		check(!bIsTimeSliceRebuilding);

		const int32 HandleValue = (int32)InOutHandle.GetValue();
		const FEntryHandle EntryHandle = Entries.GetHandle(HandleValue);

		FEntry* Entry = Entries.Get(EntryHandle);
		if (!ensure(Entry != nullptr))
		{
			return;
		}

		const ESpatialCategory OldCategory = GetCategory(Entry->Classification);
		const ESpatialCategory NewCategory = GetCategory(Classification);

		// If the category changed, we have to remove from the old first.
		if (OldCategory != NewCategory)
		{
			FSpatialAcceleration* OldAccelerationStructure = GetAccelerationStructure(Entry->Classification);
			OldAccelerationStructure->RemoveElement(HandleValue);
		}

		Entry->Aabb = Aabb;
		Entry->UserData = UserData;
		Entry->Classification = Classification;

		FSpatialAcceleration* AccelerationStructure = GetAccelerationStructure(Classification);
		AccelerationStructure->UpdateElement(HandleValue, Aabb, true);
	}

	void FLegacySpatialPartitionCollection::Remove(FSpatialHandle& InOutHandle)
	{
		check(!bIsTimeSliceRebuilding);

		const int32 HandleValue = InOutHandle.GetValue();
		// Immediately clear out the handle value for debug-ability
		InOutHandle.SetValue(INDEX_NONE);
		const FEntryHandle EntryHandle = Entries.GetHandle(HandleValue);

		const FEntry* Entry = Entries.Get(EntryHandle);
		if (!ensure(Entry != nullptr))
		{
			return;
		}

		FSpatialAcceleration* AccelerationStructure = GetAccelerationStructure(Entry->Classification);
		AccelerationStructure->RemoveElement(HandleValue);

		Entries.Destroy(EntryHandle);
	}

	void FLegacySpatialPartitionCollection::Overlap(const FOverlapQueryData& QueryData, FOverlapVisitor& Visitor, ESpatialCategoryMask CategoryMask) const
	{
		// Doing a query in the middle of time slicing won't return those results.
		check(!bIsTimeSliceRebuilding);

		FOverlapQueryRuntimeData QueryRuntimeData(QueryData);
		FLegacyVisitor LegacyVisitor(this, CategoryMask, &QueryRuntimeData, &Visitor);
		RunQueries(CategoryMask, QueryRuntimeData, LegacyVisitor);
	}

	void FLegacySpatialPartitionCollection::Raycast(const FRaycastQueryData& QueryData, FRaycastVisitor& Visitor, ESpatialCategoryMask CategoryMask) const
	{
		// Doing a query in the middle of time slicing won't return those results.
		check(!bIsTimeSliceRebuilding);

		FRaycastQueryRuntimeData QueryRuntimeData(QueryData);
		FLegacyVisitor LegacyVisitor(this, CategoryMask, &QueryRuntimeData, &Visitor);
		RunQueries(CategoryMask, QueryRuntimeData, LegacyVisitor);
	}

	void FLegacySpatialPartitionCollection::Sweep(const FSweepQueryData& QueryData, FSweepVisitor& Visitor, ESpatialCategoryMask CategoryMask) const
	{
		// Doing a query in the middle of time slicing won't return those results.
		check(!bIsTimeSliceRebuilding);

		FSweepQueryRuntimeData QueryRuntimeData(QueryData);
		FLegacyVisitor LegacyVisitor(this, CategoryMask, &QueryRuntimeData, &Visitor);
		RunQueries(CategoryMask, QueryRuntimeData, LegacyVisitor);
	}

	bool FLegacySpatialPartitionCollection::NeedsRebuilding() const
	{
		FSpatialAcceleration* Static = StaticAcceleration.Get();
		return Static->ShouldRebuild() || !Static->IsAsyncTimeSlicingComplete();
	}

	FLegacySpatialPartitionCollection::ERebuildStatus FLegacySpatialPartitionCollection::Rebuild()
	{
		// Note: We only need to do time slicing for the static tree, not dynamic.
		FStaticAabbTreeType* StaticTree = StaticAcceleration.Get();
		// Check if we need to initialize the rebuild process.
		if (!bIsTimeSliceRebuilding && StaticTree->ShouldRebuild() && StaticTree->IsAsyncTimeSlicingComplete())
		{
			// There's currently no api to rebuild the tree inline, so we have to grab all elements (global, dirty, etc...) 
			// and rebuild the tree from scratch and then swap the tree in the collection.
			TArray<TPayloadBoundsElement<int32, FReal>> Elements;
			for (FEntryHandleArray::TRangedForIterator It = Entries.begin(); It != Entries.end(); ++It)
			{
				const FEntry& Entry = *It;
				const FEntryHandle Handle = It.GetHandle();
				// Note: In static only mode this will grab everything.
				if (GetCategory(Entry.Classification) == ESpatialCategory::Static)
				{
					TPayloadBoundsElement<int32, FReal> Element;
					Element.Payload = Handle.GetIndex();
					Element.Bounds = Entry.Aabb;
					Elements.Add(Element);
				}
			}

			const FConfig::FSettings& SSettings = Config.StaticSettings;
			TUniquePtr<FStaticAabbTreeType> NewStatic = MakeUnique<FStaticAabbTreeType>(Elements, SSettings.MaxChildrenInLeaf, SSettings.MaxTreeDepth, SSettings.MaxPayloadSize, SSettings.IterationsPerTimeSlice, false, SSettings.bUseDirtyTree, SSettings.bBuildOverlapCache);
			NewStatic->ClearShouldRebuild();

			// Note: The new tree is swapped in right away which means queries won't be correct until timeslicing is done; this is how legacy currently works.
			// Timeslicing is currently only done on a copy so this doesn't affect results in the engine.
			StaticAcceleration = MoveTemp(NewStatic);
		}
		else if (!StaticTree->IsAsyncTimeSlicingComplete())
		{
			constexpr bool bForceBuildCompletion = false;
			StaticTree->ProgressAsyncTimeSlicing(bForceBuildCompletion);
		}
		bIsTimeSliceRebuilding = !StaticAcceleration->IsAsyncTimeSlicingComplete();
		return bIsTimeSliceRebuilding ? ERebuildStatus::Continue : ERebuildStatus::Finished;
	}

	FLegacySpatialPartitionCollection::FLegacyVisitor::FLegacyVisitor(const FLegacySpatialPartitionCollection* Collection, ESpatialCategoryMask CategoryMask, FOverlapQueryRuntimeData* InQueryData, FOverlapVisitor* InVisitor)
		: FLegacyVisitorBase(InQueryData, InVisitor)
		, Collection(Collection)
		, CategoryMask(CategoryMask)
	{
	}

	FLegacySpatialPartitionCollection::FLegacyVisitor::FLegacyVisitor(const FLegacySpatialPartitionCollection* Collection, ESpatialCategoryMask CategoryMask, FRaycastQueryRuntimeData* InQueryData, FRaycastVisitor* InVisitor)
		: FLegacyVisitorBase(InQueryData, InVisitor)
		, Collection(Collection)
		, CategoryMask(CategoryMask)
	{
	}

	FLegacySpatialPartitionCollection::FLegacyVisitor::FLegacyVisitor(const FLegacySpatialPartitionCollection* Collection, ESpatialCategoryMask CategoryMask, FSweepQueryRuntimeData* InQueryData, FSweepVisitor* InVisitor)
		: FLegacyVisitorBase(InQueryData, InVisitor)
		, Collection(Collection)
		, CategoryMask(CategoryMask)
	{
	}

	bool FLegacySpatialPartitionCollection::FLegacyVisitor::Overlap(const TSpatialVisitorData<int32>& Instance)
	{
		FUserDataType UserData;
		if (Collection->TryGetUserData(Instance, CategoryMask, UserData))
		{
			return OverlapInternal(UserData);
		}
		return true;
	}

	bool FLegacySpatialPartitionCollection::FLegacyVisitor::Raycast(const TSpatialVisitorData<int32>& Instance, FQueryFastData& CurData)
	{
		FUserDataType UserData;
		if (Collection->TryGetUserData(Instance, CategoryMask, UserData))
		{
			return RaycastInternal(UserData, CurData);
		}
		return true;
	}

	bool FLegacySpatialPartitionCollection::FLegacyVisitor::Sweep(const TSpatialVisitorData<int32>& Instance, FQueryFastData& CurData)
	{
		FUserDataType UserData;
		if (Collection->TryGetUserData(Instance, CategoryMask, UserData))
		{
			return SweepInternal(UserData, CurData);
		}
		return true;
	}

	void FLegacySpatialPartitionCollection::Query(const FSpatialAcceleration* SpatialAcceleration, FOverlapQueryRuntimeData& QueryRuntimeData, FSpatialVisitor& Visitor) const
	{
		SpatialAcceleration->Overlap(QueryRuntimeData.GetInputData().Aabb, Visitor);
	}

	void FLegacySpatialPartitionCollection::Query(const FSpatialAcceleration* SpatialAcceleration, FRaycastQueryRuntimeData& QueryRuntimeData, FSpatialVisitor& Visitor) const
	{
		const FRaycastQueryData& QueryData = QueryRuntimeData.GetInputData();
		SpatialAcceleration->Raycast(QueryData.Start, QueryData.Direction, QueryRuntimeData.GetCurrentLength(), Visitor);
	}

	void FLegacySpatialPartitionCollection::Query(const FSpatialAcceleration* SpatialAcceleration, FSweepQueryRuntimeData& QueryRuntimeData, FSpatialVisitor& Visitor) const
	{
		const FSweepQueryData& QueryData = QueryRuntimeData.GetInputData();
		SpatialAcceleration->Sweep(QueryData.Start, QueryData.Direction, QueryRuntimeData.GetCurrentLength(), QueryData.HalfExtents, Visitor);
	}

	bool FLegacySpatialPartitionCollection::TryGetUserData(const TSpatialVisitorData<int32>& Instance, const ESpatialCategoryMask CategoryMask, FUserDataType& OutUserData) const
	{
		const FEntryHandle EntryHandle = Entries.GetHandle(Instance.Payload);
		const FEntry* Entry = Entries.Get(EntryHandle);
		check(Entry != nullptr);
		OutUserData = Entry->UserData;

		// Test the object's original category to see if it passes the filter mask. 
		// This is necessary as one category could be coerced into another one and we want the results to respect the original category.
		const ESpatialCategoryMask TestMask = ToCategoryMask(Entry->Classification.GetCategory());
		return EnumHasAnyFlags(CategoryMask, TestMask);
	}

	ESpatialCategory FLegacySpatialPartitionCollection::GetCategory(const FSpatialClassification& Classification) const
	{
		// In static only mode, everything is coerced into the static tree.
		if (Config.bStaticOnly)
		{
			return ESpatialCategory::Static;
		}

		// We only handle static/dynamic. Coerce kinematic into dynamic.
		const ESpatialCategory Category = Classification.GetCategory();
		return (Category == ESpatialCategory::Static) ? ESpatialCategory::Static : ESpatialCategory::Dynamic;
	}

	FLegacySpatialPartitionCollection::FSpatialAcceleration* FLegacySpatialPartitionCollection::GetAccelerationStructure(const FSpatialClassification& Classification)
	{
		const ESpatialCategory Category = GetCategory(Classification);
		return (Category == ESpatialCategory::Static) ? StaticAcceleration.Get() : DynamicAcceleration.Get();
	}

	const FLegacySpatialPartitionCollection::FSpatialAcceleration* FLegacySpatialPartitionCollection::GetAccelerationStructure(const FSpatialClassification& Classification) const
	{
		const ESpatialCategory Category = GetCategory(Classification);
		return (Category == ESpatialCategory::Static) ? StaticAcceleration.Get() : DynamicAcceleration.Get();
	}
} // namespace Chaos::SpatialPartition
