// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ChaosSpatialPartitions/ISpatialPartition.h"
#include "LegacyVisitorBase.h"

#include "Chaos/ISpatialAcceleration.h"
#include "Chaos/AABBTree.h"
#include "Chaos/HandleArray.h"

#include "Templates/UniquePtr.h"

namespace Chaos::SpatialPartition
{
	// A collection that uses the legacy data structures (aabb tree). Used for testing and incremental migration.
	class FLegacySpatialPartitionCollection : public ISpatialPartition
	{
	public:
		struct FConfig
		{
			FConfig();

			struct FSettings
			{
				int32 MaxChildrenInLeaf = 500;
				int32 MaxTreeDepth = 200;
				FRealSingle MaxPayloadSize = 100000;
				int32 IterationsPerTimeSlice = 4000;
				bool bBuildOverlapCache = false;
				bool bUseDirtyTree = false;
			};

			// Controls if this has split static/dynamic. Used to emulate the legacy mode of non-split structures.
			bool bStaticOnly = false;
			FSettings StaticSettings;
			FSettings DynamicSettings;
		};

		FLegacySpatialPartitionCollection(const FConfig& InConfig = FConfig());
		FLegacySpatialPartitionCollection(FLegacySpatialPartitionCollection&&) = default;
		virtual ~FLegacySpatialPartitionCollection() override = default;

		FLegacySpatialPartitionCollection& operator=(FLegacySpatialPartitionCollection&&) = default;

		virtual void Insert(const FUserDataType& UserData, const FAABB3& Aabb, const FSpatialClassification& Classification, FSpatialHandle& OutHandle) override;
		virtual void Update(const FUserDataType& UserData, const FAABB3& Aabb, const FSpatialClassification& Classification, FSpatialHandle& InOutHandle) override;
		virtual void Remove(FSpatialHandle& InOutHandle) override;

		virtual void Overlap(const FOverlapQueryData& QueryData, FOverlapVisitor& Visitor, ESpatialCategoryMask CategoryMask = ESpatialCategoryMask::All) const override;
		virtual void Raycast(const FRaycastQueryData& QueryData, FRaycastVisitor& Visitor, ESpatialCategoryMask CategoryMask = ESpatialCategoryMask::All) const override;
		virtual void Sweep(const FSweepQueryData& QueryData, FSweepVisitor& Visitor, ESpatialCategoryMask CategoryMask = ESpatialCategoryMask::All) const override;

		virtual bool NeedsRebuilding() const;
		virtual ERebuildStatus Rebuild();

	private:
		struct FEntry
		{
			FAABB3 Aabb = FAABB3::EmptyAABB();
			FSpatialClassification Classification;
			FUserDataType UserData = INDEX_NONE;
		};

		using FEntryHandleArray = THandleArray<FEntry>;
		using FEntryHandle = typename FEntryHandleArray::FHandle;
		using FConstEntryHandle = typename FEntryHandleArray::FConstHandle;
		using FSpatialVisitor = ISpatialVisitor<int32>;
		using FSpatialAcceleration = ISpatialAcceleration<int32, FReal, 3>;
		using FStaticAabbTreeType = TAABBTree<FUserDataType, TAABBTreeLeafArray<FUserDataType>>;
		using FDynamicAabbTreeType = TAABBTree<FUserDataType, TAABBTreeLeafArray<FUserDataType>>;

		class FLegacyVisitor : public FLegacyVisitorBase
		{
		public:
			FLegacyVisitor(const FLegacySpatialPartitionCollection* Collection, ESpatialCategoryMask CategoryMask, FOverlapQueryRuntimeData* InQueryData, FOverlapVisitor* InVisitor);
			FLegacyVisitor(const FLegacySpatialPartitionCollection* Collection, ESpatialCategoryMask CategoryMask, FRaycastQueryRuntimeData* InQueryData, FRaycastVisitor* InVisitor);
			FLegacyVisitor(const FLegacySpatialPartitionCollection* Collection, ESpatialCategoryMask CategoryMask, FSweepQueryRuntimeData* InQueryData, FSweepVisitor* InVisitor);

			virtual bool Overlap(const TSpatialVisitorData<int32>& Instance) override;
			virtual bool Raycast(const TSpatialVisitorData<int32>& Instance, FQueryFastData& CurData) override;
			virtual bool Sweep(const TSpatialVisitorData<int32>& Instance, FQueryFastData& CurData) override;

			const FLegacySpatialPartitionCollection* Collection = nullptr;
			ESpatialCategoryMask CategoryMask;
		};

		void Query(const FSpatialAcceleration* SpatialAcceleration, FOverlapQueryRuntimeData& QueryRuntimeData, FSpatialVisitor& Visitor) const;
		void Query(const FSpatialAcceleration* SpatialAcceleration, FRaycastQueryRuntimeData& QueryRuntimeData, FSpatialVisitor& Visitor) const;
		void Query(const FSpatialAcceleration* SpatialAcceleration, FSweepQueryRuntimeData& QueryRuntimeData, FSpatialVisitor& Visitor) const;

		template <typename ... ArgTypes>
		void RunQueries(ESpatialCategoryMask CategoryMask, ArgTypes... Args) const
		{
			if (Config.bStaticOnly)
			{
				if (EnumHasAnyFlags(CategoryMask, ESpatialCategoryMask::All))
				{
					Query(StaticAcceleration.Get(), Args...);
				}
				return;
			}

			if (EnumHasAnyFlags(CategoryMask, ESpatialCategoryMask::Static))
			{
				Query(StaticAcceleration.Get(), Args...);
			}
			if (EnumHasAnyFlags(CategoryMask, ESpatialCategoryMask::Kinematic | ESpatialCategoryMask::Dynamic))
			{
				Query(DynamicAcceleration.Get(), Args...);
			}
		}

		bool TryGetUserData(const TSpatialVisitorData<int32>& Instance, const ESpatialCategoryMask CategoryMask, FUserDataType& OutUserData) const;
		ESpatialCategory GetCategory(const FSpatialClassification& Classification) const;
		FSpatialAcceleration* GetAccelerationStructure(const FSpatialClassification& Classification);
		const FSpatialAcceleration* GetAccelerationStructure(const FSpatialClassification& Classification) const;

		FConfig Config;
		FEntryHandleArray Entries;
		TUniquePtr<FStaticAabbTreeType> StaticAcceleration;
		TUniquePtr<FDynamicAabbTreeType> DynamicAcceleration;
		bool bIsTimeSliceRebuilding = false;
	};
} // namespace Chaos::SpatialPartition
