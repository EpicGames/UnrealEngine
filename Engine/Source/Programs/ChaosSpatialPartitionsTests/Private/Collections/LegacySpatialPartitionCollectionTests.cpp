// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "ChaosTestHarness.h"

#include "TestVisitors.h"
#include "SharedOverlapTests.h"
#include "SharedRaycastTests.h"
#include "SharedSweepTests.h"
#include "SharedSelfQueryTests.h"

#include "Collections/LegacySpatialPartitionCollection.h"
#include "SharedCollectionTests.h"

namespace Chaos::SpatialPartition
{
	// This helper class allows configuring of parameters for specific testing scenarios.
	// For instance, we can test with rebuilding our tree or leaving it in the "dirty" state.
	class FTestLegacySpatialPartitionCollection : public FLegacySpatialPartitionCollection
	{
	public:
		using FLegacySpatialPartitionCollection::FLegacySpatialPartitionCollection;

		bool bAutoRebuild = true;
		int32 DynamicTreeLeafCapacity;
		FReal DynamicTreeLeafEnlargePercent;
		FReal DynamicTreeBoundingBoxPadding;

		FTestLegacySpatialPartitionCollection(const FLegacySpatialPartitionCollection::FConfig& Config = FLegacySpatialPartitionCollection::FConfig())
			: FLegacySpatialPartitionCollection(Config)
		{
			// Many tests were written based on unit boxes. Various c-vars are defaulted to work with unreal sizes. 
			// For perf tests, clear these so they don't affect comparison tests (but cache so we can reset).
			DynamicTreeLeafCapacity = FAABBTreeCVars::DynamicTreeLeafCapacity;
			DynamicTreeLeafEnlargePercent = FAABBTreeCVars::DynamicTreeLeafEnlargePercent;
			DynamicTreeBoundingBoxPadding = FAABBTreeCVars::DynamicTreeBoundingBoxPadding;

			FAABBTreeCVars::DynamicTreeLeafCapacity = Config.DynamicSettings.MaxChildrenInLeaf;
			FAABBTreeCVars::DynamicTreeLeafEnlargePercent = 0;
			FAABBTreeCVars::DynamicTreeBoundingBoxPadding = 0;
		}
		~FTestLegacySpatialPartitionCollection()
		{
			FAABBTreeCVars::DynamicTreeLeafCapacity = DynamicTreeLeafCapacity;
			FAABBTreeCVars::DynamicTreeLeafEnlargePercent = (float)DynamicTreeLeafEnlargePercent;
			FAABBTreeCVars::DynamicTreeBoundingBoxPadding = (float)DynamicTreeBoundingBoxPadding;
		}

		void ForceRebuild()
		{
			while (ISpatialPartition::ERebuildStatus::Continue == Rebuild())
			{
			}
		}

		void TryRebuild()
		{
			if (!bAutoRebuild || !NeedsRebuilding())
			{
				return;
			}

			ForceRebuild();
		}

		static FTestLegacySpatialPartitionCollection* CreateForPerfTest()
		{
			FTestLegacySpatialPartitionCollection* Result = new FTestLegacySpatialPartitionCollection();
			Result->bAutoRebuild = false;
			return Result;
		};
	};

	template <>
	void InsertObject(FTestLegacySpatialPartitionCollection& SpatialPartition, FObjectData& Object)
	{
		SpatialPartition.Insert(Object.UserData, Object.Aabb, Object.Classification, Object.Handle);
		SpatialPartition.TryRebuild();
	}

	template <>
	void UpdateObject(FTestLegacySpatialPartitionCollection& SpatialPartition, FObjectData& Object)
	{
		SpatialPartition.Update(Object.UserData, Object.Aabb, Object.Classification, Object.Handle);
		SpatialPartition.TryRebuild();
	}

	template <>
	void RemoveObject(FTestLegacySpatialPartitionCollection& SpatialPartition, FObjectData& Object)
	{
		SpatialPartition.Remove(Object.Handle);
		SpatialPartition.TryRebuild();
	}

	template <>
	void BuildFromObjects(FTestLegacySpatialPartitionCollection& SpatialPartition, TArray<FObjectData>& Objects)
	{
		for (FObjectData& Object : Objects)
		{
			SpatialPartition.Insert(Object.UserData, Object.Aabb, Object.Classification, Object.Handle);
		}
		SpatialPartition.TryRebuild();
	}

	template <>
	void Rebuild(FTestLegacySpatialPartitionCollection& SpatialPartition)
	{
		SpatialPartition.ForceRebuild();
	}

	TEST_CASE("LegacySpatialPartitionCollection - Build", "[Chaos][LegacySpatialPartitionCollection][spatial-partition]")
	{
		FTestLegacySpatialPartitionCollection::FConfig Config;
		Config.bStaticOnly = GENERATE(false, true);
		FTestLegacySpatialPartitionCollection Collection(Config);
		Collection.bAutoRebuild = GENERATE(false, true);
		TestCollectionBasicBuild(Collection);
	}

	TEST_CASE("LegacySpatialPartitionCollection - Build Large", "[Chaos][LegacySpatialPartitionCollection][spatial-partition]")
	{
		// Test with a large number of objects to enforce time slicing to take a while.
		constexpr int32 ObjectCount = 10000;
		FTestLegacySpatialPartitionCollection::FConfig Config;
		Config.bStaticOnly = true;
		FTestLegacySpatialPartitionCollection Collection(Config);
		Collection.bAutoRebuild = true;

		const TArray<FSpatialClassification> Classifications = GetClassifications();
		TArray<FObjectData> Objects;
		BuildObjectList(Objects, ObjectCount, Classifications);
		BuildFromObjects(Collection, Objects);

		FOverlapQueryData QueryData;
		QueryData.Aabb = Objects[0].Aabb;
		QueryData.Aabb.GrowToInclude(Objects.Last().Aabb);
		FTestOverlapCollectorVisitor Visitor;
		Collection.Overlap(QueryData, Visitor);
		CHECK(Visitor.Results.Num() == Objects.Num());
	}

	TEST_CASE("LegacySpatialPartitionCollection - Insert", "[Chaos][LegacySpatialPartitionCollection][spatial-partition]")
	{
		FTestLegacySpatialPartitionCollection::FConfig Config;
		Config.bStaticOnly = GENERATE(false, true);
		FTestLegacySpatialPartitionCollection Collection(Config);
		Collection.bAutoRebuild = GENERATE(false, true);
		TestCollectionBasicInsert(Collection);
	}

	TEST_CASE("LegacySpatialPartitionCollection - Update - Same Category", "[Chaos][LegacySpatialPartitionCollection][spatial-partition]")
	{
		FTestLegacySpatialPartitionCollection::FConfig Config;
		Config.bStaticOnly = GENERATE(false, true);
		FTestLegacySpatialPartitionCollection Collection(Config);
		Collection.bAutoRebuild = GENERATE(false, true);
		TestCollectionBasicUpdateSameCategory(Collection);
	}

	TEST_CASE("LegacySpatialPartitionCollection - Update - Different Category", "[Chaos][LegacySpatialPartitionCollection][spatial-partition]")
	{
		FTestLegacySpatialPartitionCollection::FConfig Config;
		Config.bStaticOnly = GENERATE(false, true);
		FTestLegacySpatialPartitionCollection Collection(Config);
		Collection.bAutoRebuild = GENERATE(false, true);
		TestCollectionBasicUpdateDifferentCategory(Collection);
	}

	TEST_CASE("LegacySpatialPartitionCollection - Remove", "[Chaos][LegacySpatialPartitionCollection][spatial-partition]")
	{
		FTestLegacySpatialPartitionCollection::FConfig Config;
		Config.bStaticOnly = GENERATE(false, true);
		FTestLegacySpatialPartitionCollection Collection(Config);
		Collection.bAutoRebuild = GENERATE(false, true);
		TestCollectionBasicRemove(Collection);
	}

	TEST_CASE("LegacySpatialPartitionCollection - Insert Perf", "[Chaos][LegacySpatialPartitionCollection][spatial-partition][!benchmark]")
	{
		TestCollectionBasicInsertPerf<FTestLegacySpatialPartitionCollection>(FTestLegacySpatialPartitionCollection::CreateForPerfTest);
	}

	TEST_CASE("LegacySpatialPartitionCollection - Update Perf", "[Chaos][LegacySpatialPartitionCollection][spatial-partition][!benchmark]")
	{
		TestCollectionBasicUpdatePerf<FTestLegacySpatialPartitionCollection>(FTestLegacySpatialPartitionCollection::CreateForPerfTest);
	}

	TEST_CASE("LegacySpatialPartitionCollection - Remove Perf", "[Chaos][LegacySpatialPartitionCollection][spatial-partition][!benchmark]")
	{
		TestCollectionBasicRemovePerfWithSplitClassifications<FTestLegacySpatialPartitionCollection>(FTestLegacySpatialPartitionCollection::CreateForPerfTest);
	}

	TEST_CASE("LegacySpatialPartitionCollection - Rebuild Perf", "[Chaos][LegacySpatialPartitionCollection][spatial-partition][!benchmark]")
	{
		TestCollectionBasicRebuildPerf<FTestLegacySpatialPartitionCollection>(FTestLegacySpatialPartitionCollection::CreateForPerfTest);
	}

	TEST_CASE("LegacySpatialPartitionCollection - Query Overlap Perf", "[Chaos][LegacySpatialPartitionCollection][spatial-partition][!benchmark]")
	{
		FTestLegacySpatialPartitionCollection Collection;
		Collection.bAutoRebuild = false;
		TestCollectionBasicOverlapPerf(Collection);
	}
} // namespace Chaos::SpatialPartition
