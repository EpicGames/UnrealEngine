// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "ChaosTestHarness.h"

#include "SharedCollectionTests.h"

#include "ChaosSpatialPartitions/Collections/NSquaredAabbCollection.h"

namespace Chaos::SpatialPartition
{
	static FNSquaredAabbCollection* CreateNSquaredCollectionForPerfTest()
	{
		return new FNSquaredAabbCollection();
	};

	template <>
	void InsertObject(FNSquaredAabbCollection& SpatialPartition, FObjectData& Object)
	{
		SpatialPartition.Insert(Object.UserData, Object.Aabb, Object.Classification, Object.Handle);
	}

	template <>
	void UpdateObject(FNSquaredAabbCollection& SpatialPartition, FObjectData& Object)
	{
		SpatialPartition.Update(Object.UserData, Object.Aabb, Object.Classification, Object.Handle);
	}

	template <>
	void RemoveObject(FNSquaredAabbCollection& SpatialPartition, FObjectData& Object)
	{
		SpatialPartition.Remove(Object.Handle);
	}

	template <>
	void BuildFromObjects(FNSquaredAabbCollection& SpatialPartition, TArray<FObjectData>& Objects)
	{
		for (FObjectData& Object : Objects)
		{
			SpatialPartition.Insert(Object.UserData, Object.Aabb, Object.Classification, Object.Handle);
		}
	}

	TEST_CASE("NSquaredAabbCollection - Build", "[Chaos][NSquaredAabbCollection][spatial-partition]")
	{
		FNSquaredAabbCollection Collection;
		TestCollectionBasicBuild(Collection);
	}

	TEST_CASE("NSquaredAabbCollection - Insert", "[Chaos][NSquaredAabbCollection][spatial-partition]")
	{
		FNSquaredAabbCollection Collection;
		TestCollectionBasicInsert(Collection);
	}

	TEST_CASE("NSquaredAabbCollection - Update - Same Category", "[Chaos][NSquaredAabbCollection][spatial-partition]")
	{
		FNSquaredAabbCollection Collection;
		TestCollectionBasicUpdateSameCategory(Collection);
	}

	TEST_CASE("NSquaredAabbCollection - Update - Different Category", "[Chaos][NSquaredAabbCollection][spatial-partition]")
	{
		FNSquaredAabbCollection Collection;
		TestCollectionBasicUpdateDifferentCategory(Collection);
	}

	TEST_CASE("NSquaredAabbCollection - Remove", "[Chaos][NSquaredAabbCollection][spatial-partition]")
	{
		FNSquaredAabbCollection Collection;
		TestCollectionBasicRemove(Collection);
	}

	TEST_CASE("NSquaredAabbCollection - Insert Perf", "[Chaos][NSquaredAabbCollection][spatial-partition][!benchmark]")
	{
		TestCollectionBasicInsertPerf<FNSquaredAabbCollection>(CreateNSquaredCollectionForPerfTest);
	}

	TEST_CASE("NSquaredAabbCollection - Update Perf", "[Chaos][NSquaredAabbCollection][spatial-partition][!benchmark]")
	{
		TestCollectionBasicUpdatePerf<FNSquaredAabbCollection>(CreateNSquaredCollectionForPerfTest);
	}

	TEST_CASE("NSquaredAabbCollection - Remove Perf", "[Chaos][NSquaredAabbCollection][spatial-partition][!benchmark]")
	{
		TestCollectionBasicRemovePerfWithSplitClassifications<FNSquaredAabbCollection>(CreateNSquaredCollectionForPerfTest);
	}

	TEST_CASE("NSquaredAabbCollection - Query Overlap Perf", "[Chaos][NSquaredAabbCollection][spatial-partition][!benchmark]")
	{
		FNSquaredAabbCollection Collection;
		TestCollectionBasicOverlapPerf(Collection);
	}
} // namespace Chaos::SpatialPartition
