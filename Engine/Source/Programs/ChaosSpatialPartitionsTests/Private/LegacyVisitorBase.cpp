// Copyright Epic Games, Inc. All Rights Reserved.

#include "LegacyVisitorBase.h"

namespace Chaos::SpatialPartition
{
	FLegacyVisitorBase::FLegacyVisitorBase(FOverlapQueryRuntimeData* InQueryData, FOverlapVisitor* InVisitor)
		: OverlapQueryData(InQueryData), OverlapVisitor(InVisitor)
	{
		check(InQueryData != nullptr);
		check(InVisitor != nullptr);
	}

	FLegacyVisitorBase::FLegacyVisitorBase(FRaycastQueryRuntimeData* InQueryData, FRaycastVisitor* InVisitor)
		: RaycastQueryData(InQueryData), RaycastVisitor(InVisitor)
	{
		check(InQueryData != nullptr);
		check(InVisitor != nullptr);
	}

	FLegacyVisitorBase::FLegacyVisitorBase(FSweepQueryRuntimeData* InQueryData, FSweepVisitor* InVisitor)
		: SweepQueryData(InQueryData), SweepVisitor(InVisitor)
	{
		check(InQueryData != nullptr);
		check(InVisitor != nullptr);
	}

	bool FLegacyVisitorBase::OverlapInternal(const FUserDataType& UserData)
	{
		check(OverlapVisitor != nullptr);
		check(OverlapQueryData != nullptr);

		const EVisitResult VisitResult = OverlapVisitor->Visit(UserData, *OverlapQueryData);
		return VisitResult == EVisitResult::Continue;
	}

	bool FLegacyVisitorBase::RaycastInternal(const FUserDataType& UserData, FQueryFastData& CurData)
	{
		check(RaycastVisitor != nullptr);
		check(RaycastQueryData != nullptr);

		const EVisitResult VisitResult = RaycastVisitor->Visit(UserData, *RaycastQueryData);
		// Update the length in case the user modified it.
		CurData.SetLength(RaycastQueryData->GetCurrentLength());
		return VisitResult == EVisitResult::Continue;
	}

	bool FLegacyVisitorBase::SweepInternal(const FUserDataType& UserData, FQueryFastData& CurData)
	{
		check(SweepVisitor != nullptr);
		check(SweepQueryData != nullptr);

		const EVisitResult VisitResult = SweepVisitor->Visit(UserData, *SweepQueryData);
		// Update the length in case the user modified it.
		CurData.SetLength(SweepQueryData->GetCurrentLength());
		return VisitResult == EVisitResult::Continue;
	}
} // namespace Chaos::SpatialPartition
