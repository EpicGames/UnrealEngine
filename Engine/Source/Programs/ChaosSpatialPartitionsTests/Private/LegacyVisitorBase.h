// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Chaos/ISpatialAcceleration.h"
#include "ChaosSpatialPartitions/Visitors.h"

namespace Chaos::SpatialPartition
{
	// This class wraps the new visitors with the old visitor API.
	// The old API is not directly implemented as this allows any individual visitor to do 
	// their own mapping of the legacy payload to the new user data.
	class FLegacyVisitorBase : public ISpatialVisitor<int32>
	{
	public:
		FLegacyVisitorBase(FOverlapQueryRuntimeData* InQueryData, FOverlapVisitor* InVisitor);
		FLegacyVisitorBase(FRaycastQueryRuntimeData* InQueryData, FRaycastVisitor* InVisitor);
		FLegacyVisitorBase(FSweepQueryRuntimeData* InQueryData, FSweepVisitor* InVisitor);

		bool OverlapInternal(const FUserDataType& UserData);
		bool RaycastInternal(const FUserDataType& UserData, FQueryFastData& CurData);
		bool SweepInternal(const FUserDataType& UserData, FQueryFastData& CurData);

		FOverlapQueryRuntimeData* OverlapQueryData = nullptr;
		FOverlapVisitor* OverlapVisitor = nullptr;

		FRaycastQueryRuntimeData* RaycastQueryData = nullptr;
		FRaycastVisitor* RaycastVisitor = nullptr;

		FSweepQueryRuntimeData* SweepQueryData = nullptr;
		FSweepVisitor* SweepVisitor = nullptr;
	};
} // namespace Chaos::SpatialPartition
