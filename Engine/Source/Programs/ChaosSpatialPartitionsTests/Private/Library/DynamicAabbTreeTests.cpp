// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "ChaosTestHarness.h"

#include "AabbTreeTestHelpers.h"
#include "TestVisitors.h"
#include "SharedOverlapTests.h"
#include "SharedRaycastTests.h"
#include "SharedSweepTests.h"
#include "SharedSelfQueryTests.h"

#include "ChaosSpatialPartitions/Library/DynamicAabbTree.h"

namespace Chaos::SpatialPartition
{
	void DumpAndCompare(const FDynamicAabbTree& Tree, const TArray<FAabbTreeNode>& ExpectedNodes, const int32 ExpectedRoot)
	{
		TArray<FAabbTreeNode> ActualNodes;
		int32 ActualRoot;
		Tree.Dump(ActualNodes, ActualRoot);
		CHECKED_IF(IsValidNodeIndex(ExpectedRoot) == IsValidNodeIndex(ActualRoot))
		{
			if (IsValidNodeIndex(ExpectedRoot))
			{
				Compare(ExpectedNodes, ExpectedRoot, ActualNodes, ActualRoot);
			}
		}
	}

	TEST_CASE("DynamicAabbTree - Overlap - Basic", "[Chaos][DynamicAabbTree][spatial-partition]")
	{
		FDynamicAabbTree SpatialPartition;
		SECTION("Build")
		{
			TestOverlapBuildBasic(SpatialPartition);
		}
		SECTION("Insert")
		{
			TestOverlapInsert(SpatialPartition);
		}
		SECTION("Remove")
		{
			TestOverlapRemove(SpatialPartition);
		}
		SECTION("Update: Basic")
		{
			TestOverlapUpdateBasic(SpatialPartition);
		}
		SECTION("Update: VisitResult Early Termination")
		{
			TestOverlapWithEarlyVisitResultTermination(SpatialPartition);
		}
	}

	TEST_CASE("DynamicAabbTree - Overlap - Performance", "[Chaos][DynamicAabbTree][spatial-partition][!benchmark]")
	{
		FDynamicAabbTree SpatialPartition;
		SECTION("Overlap: Empty")
		{
			TestOverlapPerformanceEmpty(SpatialPartition);
		}
		SECTION("Overlap: Basic")
		{
			TestOverlapPerformanceBasic(SpatialPartition, 15, 15, 15);
		}
	}

	TEST_CASE("DynamicAabbTree - Raycast - Basic", "[Chaos][DynamicAabbTree][spatial-partition]")
	{
		FDynamicAabbTree SpatialPartition;
		SECTION("Build")
		{
			TestRaycastBuildBasic(SpatialPartition);
		}
		SECTION("Insert")
		{
			TestRaycastInsert(SpatialPartition);
		}
		SECTION("Remove")
		{
			TestRaycastRemove(SpatialPartition);
		}
		SECTION("Update: Basic")
		{
			TestRaycastUpdateBasic(SpatialPartition);
		}
		SECTION("Update: VisitResult Early Termination")
		{
			TestRaycastWithEarlyVisitResultTermination(SpatialPartition);
		}
		SECTION("Update: Length Early Termination")
		{
			TestRaycastWithEarlyLengthTermination(SpatialPartition);
		}
	}

	TEST_CASE("DynamicAabbTree - Raycast - Performance", "[Chaos][DynamicAabbTree][spatial-partition][!benchmark]")
	{
		FDynamicAabbTree SpatialPartition;
		SECTION("Raycast: Empty")
		{
			TestRaycastPerformanceEmpty(SpatialPartition);
		}
		SECTION("Raycast: Basic")
		{
			TestRaycastPerformanceBasic(SpatialPartition);
		}
		SECTION("Raycast: First Hit")
		{
			TestRaycastPerformanceFirstHit(SpatialPartition);
		}
	}

	TEST_CASE("DynamicAabbTree - Sweep - Basic", "[Chaos][DynamicAabbTree][spatial-partition]")
	{
		FDynamicAabbTree SpatialPartition;
		SECTION("Build")
		{
			TestSweepBuildBasic(SpatialPartition);
		}
		SECTION("Insert")
		{
			TestSweepInsert(SpatialPartition);
		}
		SECTION("Remove")
		{
			TestSweepRemove(SpatialPartition);
		}
		SECTION("Update: Basic")
		{
			TestSweepUpdateBasic(SpatialPartition);
		}
		SECTION("Update: VisitResult Early Termination")
		{
			TestSweepWithEarlyVisitResultTermination(SpatialPartition);
		}
		SECTION("Update: Length Early Termination")
		{
			TestSweepWithEarlyLengthTermination(SpatialPartition);
		}
	}

	TEST_CASE("DynamicAabbTree - Sweep - Performance", "[Chaos][DynamicAabbTree][spatial-partition][!benchmark]")
	{
		FDynamicAabbTree SpatialPartition;
		SECTION("Sweep: Empty")
		{
			TestSweepPerformanceEmpty(SpatialPartition);
		}
		SECTION("Sweep: Basic")
		{
			TestSweepPerformanceBasic(SpatialPartition);
		}
		SECTION("Sweep: First Hit")
		{
			TestSweepPerformanceFirstHit(SpatialPartition);
		}
	}

	TEST_CASE("DynamicAabbTree - SelfQuery - Basic", "[Chaos][DynamicAabbTree][spatial-partition]")
	{
		FDynamicAabbTree SpatialPartition;
		SECTION("Build")
		{
			TestSelfQueryBuildBasic(SpatialPartition);
		}
		SECTION("Insert")
		{
			TestSelfQueryInsert(SpatialPartition);
		}
		SECTION("Remove")
		{
			TestSelfQueryRemove(SpatialPartition);
		}
		SECTION("Update: Basic")
		{
			TestSelfQueryUpdateBasic(SpatialPartition);
		}
		SECTION("Update: Movement")
		{
			TestSelfQueryUpdateMovement(SpatialPartition);
		}
	}

	TEST_CASE("DynamicAabbTree - SelfQuery - Performance", "[Chaos][DynamicAabbTree][spatial-partition][!benchmark]")
	{
		FDynamicAabbTree SpatialPartition;
		SECTION("Self Query: Empty")
		{
			TestSelfQueryPerformanceEmpty(SpatialPartition);
		}
		SECTION("Self Query: Basic")
		{
			TestSelfQueryPerformanceBasic(SpatialPartition, 15, 15, 15);
		}
	}

	TEST_CASE("DynamicAabbTree - Insert 1 Node", "[Chaos][DynamicAabbTree][spatial-partition]")
	{
		const FVec3 AabbSize = FVec3(1);

		TArray<FAabbTreeNode> ExpectedNodes;
		BuildNodeList(ExpectedNodes, 1, 1, 1, FVec3::Zero(), AabbSize);
		FSpatialHandle Handle;
		int32 ExpectedRoot = 0;

		FDynamicAabbTree Tree;
		Tree.Insert(ExpectedNodes[0].UserData, ExpectedNodes[0].Aabb, Handle);

		SECTION("Insert")
		{
			DumpAndCompare(Tree, ExpectedNodes, ExpectedRoot);
		}
		SECTION("Update")
		{
			ExpectedNodes[0].UserData = 2;
			ExpectedNodes[0].Aabb = FAABB3(FVec3(2, 2, 2), FVec3(3, 3, 3));

			Tree.Update(ExpectedNodes[0].UserData, ExpectedNodes[0].Aabb, Handle);

			DumpAndCompare(Tree, ExpectedNodes, ExpectedRoot);
		}
		SECTION("Remove")
		{
			Tree.Remove(Handle);

			TArray<FAabbTreeNode> ActualNodes;
			int32 ActualRoot;
			Tree.Dump(ActualNodes, ActualRoot);
			CHECK(INDEX_NONE == ActualRoot);
		}
	}

	TEST_CASE("DynamicAabbTree - Insert 2 Nodes", "[Chaos][DynamicAabbTree][spatial-partition]")
	{
		const FVec3 AabbSize = FVec3(1, 1, 1);

		TArray<FAabbTreeNode> ExpectedNodes;
		BuildNodeList(ExpectedNodes, 2, 1, 1, FVec3::Zero(), AabbSize);
		TArray<FSpatialHandle> Handles;
		Handles.SetNum(ExpectedNodes.Num());

		int32 ExpectedRoot = BuildAndAppendNode(ExpectedNodes, 0, 1);

		FDynamicAabbTree Tree;
		Tree.Insert(ExpectedNodes[0].UserData, ExpectedNodes[0].Aabb, Handles[0]);
		// Tree: A(0, 1)
		Tree.Insert(ExpectedNodes[1].UserData, ExpectedNodes[1].Aabb, Handles[1]);

		SECTION("Insert")
		{
			DumpAndCompare(Tree, ExpectedNodes, ExpectedRoot);
		}
		SECTION("Update 0")
		{
			ExpectedNodes[0].Aabb = BuildAabbMinExtents(FVec3(2, 0, 0), AabbSize);
			ExpectedNodes[ExpectedRoot].Aabb = Union(ExpectedNodes[0].Aabb, ExpectedNodes[1].Aabb);
			// Since update removes then re-inserts, the nodes will be swapped on the root (effectively inserting 1 then 0)
			Swap(ExpectedNodes[ExpectedRoot].Left, ExpectedNodes[ExpectedRoot].Right);

			Tree.Update(ExpectedNodes[0].UserData, ExpectedNodes[0].Aabb, Handles[0]);

			DumpAndCompare(Tree, ExpectedNodes, ExpectedRoot);
		}
		SECTION("Update 1")
		{
			ExpectedNodes[1].Aabb = BuildAabbMinExtents(FVec3(-1, 0, 0), AabbSize);
			ExpectedNodes[ExpectedRoot].Aabb = Union(ExpectedNodes[0].Aabb, ExpectedNodes[1].Aabb);

			Tree.Update(ExpectedNodes[1].UserData, ExpectedNodes[1].Aabb, Handles[1]);

			DumpAndCompare(Tree, ExpectedNodes, ExpectedRoot);
		}
		SECTION("Remove 0")
		{
			ExpectedRoot = 1;
			Tree.Remove(Handles[0]);

			DumpAndCompare(Tree, ExpectedNodes, ExpectedRoot);
		}
		SECTION("Remove 1")
		{
			ExpectedRoot = 0;
			Tree.Remove(Handles[1]);

			DumpAndCompare(Tree, ExpectedNodes, ExpectedRoot);
		}
	}

	TEST_CASE("DynamicAabbTree - Insert 4 Nodes", "[Chaos][DynamicAabbTree][spatial-partition]")
	{
		const FVec3 AabbSize = FVec3(1, 1, 1);
		// Build 4 objects laid out like:
		// 01
		// 
		// 23
		// This ensures a consistent grouping with no tie scores
		TArray<FAabbTreeNode> ExpectedNodes;
		ExpectedNodes.Emplace(FAabbTreeNode{ .Aabb = BuildAabbMinExtents(FVec3(0, 2, 0), AabbSize), .UserData = 0 });
		ExpectedNodes.Emplace(FAabbTreeNode{ .Aabb = BuildAabbMinExtents(FVec3(1, 2, 0), AabbSize), .UserData = 1 });
		ExpectedNodes.Emplace(FAabbTreeNode{ .Aabb = BuildAabbMinExtents(FVec3(0, 0, 0), AabbSize), .UserData = 2 });
		ExpectedNodes.Emplace(FAabbTreeNode{ .Aabb = BuildAabbMinExtents(FVec3(1, 0, 0), AabbSize), .UserData = 3 });

		TArray<FSpatialHandle> Handles;
		Handles.SetNum(ExpectedNodes.Num());

		FDynamicAabbTree Tree;
		// Tree: 0
		Tree.Insert(ExpectedNodes[0].UserData, ExpectedNodes[0].Aabb, Handles[0]);
		// Tree: A(0, 1)
		Tree.Insert(ExpectedNodes[1].UserData, ExpectedNodes[1].Aabb, Handles[1]);
		// Tree: B(A(0, 1), 2)
		Tree.Insert(ExpectedNodes[2].UserData, ExpectedNodes[2].Aabb, Handles[2]);

		const int32 Parent01 = BuildAndAppendNode(ExpectedNodes, 0, 1);
		const int32 Parent01_2 = BuildAndAppendNode(ExpectedNodes, Parent01, 2);
		SECTION("Insert: [0, 1, 2]")
		{
			DumpAndCompare(Tree, ExpectedNodes, Parent01_2);
		}

		// Tree: B(A(0, 1), C(2, 3))
		Tree.Insert(ExpectedNodes[3].UserData, ExpectedNodes[3].Aabb, Handles[3]);

		const int32 Parent23 = BuildAndAppendNode(ExpectedNodes, 2, 3);
		const int32 Parent01_23 = BuildAndAppendNode(ExpectedNodes, Parent01, Parent23);

		SECTION("Insert: [0, 1, 2, 3]")
		{
			DumpAndCompare(Tree, ExpectedNodes, Parent01_23);
		}
		SECTION("Remove 0")
		{
			// Tree: A(1, C(2, 3))
			Tree.Remove(Handles[0]);
			int32 Parent1_23 = BuildAndAppendNode(ExpectedNodes, 1, Parent23);

			DumpAndCompare(Tree, ExpectedNodes, Parent1_23);
		}
		SECTION("Remove 1")
		{
			// Tree: A(0, C(2, 3))
			Tree.Remove(Handles[1]);
			int32 Parent0_23 = BuildAndAppendNode(ExpectedNodes, 0, Parent23);

			DumpAndCompare(Tree, ExpectedNodes, Parent0_23);
		}
		SECTION("Remove 2")
		{
			// Tree: A(B(0, 1),3)
			Tree.Remove(Handles[2]);
			int32 Parent01_3 = BuildAndAppendNode(ExpectedNodes, Parent01, 3);

			DumpAndCompare(Tree, ExpectedNodes, Parent01_3);
		}
		SECTION("Remove 3")
		{
			// Tree: A(B(0, 1), 2)
			Tree.Remove(Handles[3]);

			DumpAndCompare(Tree, ExpectedNodes, Parent01_2);
		}
	}

	TEST_CASE("DynamicAabbTree - Expanded Aabb", "[Chaos][DynamicAabbTree][spatial-partition]")
	{
		const FVec3 AabbSize = FVec3(1);
		const int32 ExpectedRoot = 0;

		TArray<FAabbTreeNode> ExpectedNodes;
		BuildNodeList(ExpectedNodes, 1, 1, 1, FVec3::Zero(), AabbSize);
		FSpatialHandle Handle;

		FAABB3 RealAabb = FAABB3(FVec3(0), FVec3(0.1f));
		FAABB3 ExpandedAabb = ExpectedNodes[0].Aabb;

		FDynamicAabbTree Tree;

		// Insert with a different real aabb from expanded aabb. We expect the expanded aabb to be all that's returned.
		Tree.Insert(ExpectedNodes[0].UserData, RealAabb, ExpandedAabb, Handle);

		SECTION("Insert")
		{
			DumpAndCompare(Tree, ExpectedNodes, ExpectedRoot);
		}
		SECTION("Update - Expanded and real within bounds")
		{
			// Calling update with the aabb contained shouldn't change anything.
			RealAabb = FAABB3(FVec3(0.4f), FVec3(0.6f));
			ExpandedAabb = FAABB3(FVec3(0.1f), FVec3(0.9f));
			Tree.Update(ExpectedNodes[0].UserData, RealAabb, ExpandedAabb, Handle);

			DumpAndCompare(Tree, ExpectedNodes, ExpectedRoot);
		}
		SECTION("Update - Expanded outside bounds but real inside")
		{
			// The expanded aabb being out of bounds shouldn't change anything since the real is still contained.
			ExpandedAabb = FAABB3(FVec3(0.1f), FVec3(1.1f));
			Tree.Update(ExpectedNodes[0].UserData, RealAabb, ExpandedAabb, Handle);

			DumpAndCompare(Tree, ExpectedNodes, ExpectedRoot);
		}
		SECTION("Update - Expanded and real outside bounds")
		{
			// With the real aabb being out of bounds of the original expanded, the object should be updated.
			ExpectedNodes[0].Aabb = ExpandedAabb = FAABB3(FVec3(0.1f), FVec3(1.1f));
			RealAabb = FAABB3(FVec3(0.9f), FVec3(1.1f));
			Tree.Update(ExpectedNodes[0].UserData, RealAabb, ExpandedAabb, Handle);

			DumpAndCompare(Tree, ExpectedNodes, ExpectedRoot);
		}
	}

	TEST_CASE("DynamicAabbTree - Rotation", "[Chaos][DynamicAabbTree][spatial-partition]")
	{
		const FVec3 AabbSize = FVec3(1, 1, 1);
		// This scenario is carefully laid out to produce a rotation but have no ties in costs so the test is stable.
		// Build a set of aabbs laid out like:
		//  65
		//   4
		//
		// 1 3
		// 0 2
		TArray<FAabbTreeNode> ExpectedNodes
		{
			FAabbTreeNode{.Aabb = BuildAabbMinExtents(FVec3(0, 0, 0), AabbSize), .UserData = 0 },
			FAabbTreeNode{.Aabb = BuildAabbMinExtents(FVec3(0, 1, 0), AabbSize), .UserData = 1 },
			FAabbTreeNode{.Aabb = BuildAabbMinExtents(FVec3(2, 0, 0), AabbSize), .UserData = 2 },
			FAabbTreeNode{.Aabb = BuildAabbMinExtents(FVec3(2, 1, 0), AabbSize), .UserData = 3 },
			FAabbTreeNode{.Aabb = BuildAabbMinExtents(FVec3(2, 3, 0), AabbSize), .UserData = 4 },
			FAabbTreeNode{.Aabb = BuildAabbMinExtents(FVec3(2, 4, 0), AabbSize), .UserData = 5 },
			// Note: 6 is slightly shifted to the left so it will group with F vs. 5
			FAabbTreeNode{.Aabb = BuildAabbMinExtents(FVec3(0.9f, 4, 0), AabbSize), .UserData = 6 },
		};
		TArray<FSpatialHandle> Handles;
		Handles.SetNum(ExpectedNodes.Num());
		// Before inserting 6, the tree should be
		//     A
		//  B      C
		// 0 1   D   F
		//      2 3 4 5
		FDynamicAabbTree Tree;
		for (int32 I = 0; I < 6; ++I)
		{
			Tree.Insert(ExpectedNodes[I].UserData, ExpectedNodes[I].Aabb, Handles[I]);
		}
		const int32 Parent01 = BuildAndAppendNode(ExpectedNodes, 0, 1);
		const int32 Parent23 = BuildAndAppendNode(ExpectedNodes, 2, 3);
		const int32 Parent45 = BuildAndAppendNode(ExpectedNodes, 4, 5);
		const int32 Parent23_45 = BuildAndAppendNode(ExpectedNodes, Parent23, Parent45);
		const int32 OldRoot = BuildAndAppendNode(ExpectedNodes, Parent01, Parent23_45);
		DumpAndCompare(Tree, ExpectedNodes, OldRoot);

		// 6 most naturally will group with 5, producing the tree:
		//      A
		//  B       C
		// 0 1   D     E
		//      2 3  F   6
		//          4 5
		// This tree can be optimized by the performing a rotation on A that swaps B and E:
		//        A
		//    E       C
		//  F   6   D   B
		// 4 5     2 3 0 1
		Tree.Insert(ExpectedNodes[6].UserData, ExpectedNodes[6].Aabb, Handles[6]);

		const int32 Parent45_6 = BuildAndAppendNode(ExpectedNodes, Parent45, 6);
		const int32 Parent23_01 = BuildAndAppendNode(ExpectedNodes, Parent23, Parent01);
		const int32 ExpectedRoot = BuildAndAppendNode(ExpectedNodes, Parent45_6, Parent23_01);
		DumpAndCompare(Tree, ExpectedNodes, ExpectedRoot);
	}
} // namespace Chaos::SpatialPartition
