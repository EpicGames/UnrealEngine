// Copyright Epic Games, Inc. All Rights Reserved.

#include "LegacyAabbTree.h"

namespace Chaos::SpatialPartition
{
	FLegacyAabbTreeCVarCacheContext::FLegacyAabbTreeCVarCacheContext()
	{
		DynamicTreeLeafCapacity = FAABBTreeCVars::DynamicTreeLeafCapacity;
		DynamicTreeLeafEnlargePercent = FAABBTreeCVars::DynamicTreeLeafEnlargePercent;
		DynamicTreeBoundingBoxPadding = FAABBTreeCVars::DynamicTreeBoundingBoxPadding;
	}

	FLegacyAabbTreeCVarCacheContext::~FLegacyAabbTreeCVarCacheContext()
	{
		FAABBTreeCVars::DynamicTreeLeafCapacity = DynamicTreeLeafCapacity;
		FAABBTreeCVars::DynamicTreeLeafEnlargePercent = (float)DynamicTreeLeafEnlargePercent;
		FAABBTreeCVars::DynamicTreeBoundingBoxPadding = (float)DynamicTreeBoundingBoxPadding;
	}

	FLegacyAabbTree::FLegacyAabbTree(const FConfig& Config)
		: Config(Config)
		, Tree(AABBDynamicTreeType::EmptyInit(), Config.MaxChildrenInLeaf, Config.MaxTreeDepth, Config.MaxPayloadBounds, Config.MaxNumToProcess, Config.bDynamicTree, false, true)
	{
		// There are several values we need to override that are cvars and any constructor values are ignored. For accurate test comparisons, we need to make sure these are set.
		FAABBTreeCVars::DynamicTreeLeafCapacity = Config.MaxChildrenInLeaf;
		FAABBTreeCVars::DynamicTreeLeafEnlargePercent = 0;
		FAABBTreeCVars::DynamicTreeBoundingBoxPadding = 0;
	}

	void FLegacyAabbTree::Insert(const FUserDataType& UserData, const FAABB3& Aabb, FSpatialHandle& OutHandle)
	{
		const int32 Index = Entries.Emplace(FEntry{ .Aabb = Aabb, .UserData = UserData, .bIsUsed = true });
		Tree.UpdateElement(Index, Aabb, true);
		OutHandle.SetValue(Index);
	}

	void FLegacyAabbTree::Update(const FUserDataType& UserData, const FAABB3& Aabb, FSpatialHandle& InOutHandle)
	{
		const int32 Index = (int32)InOutHandle.GetValue();
		check(Entries.IsValidIndex(Index));
		// In the new API we allow changing this, however the old tree is keyed off this. 
		// Changing the user data would not be equivalent to an update, but a new insert. We don't care about this currently so just assert.
		check(UserData == Entries[Index].UserData);
		check(Entries[Index].bIsUsed);
		Entries[Index].Aabb = Aabb;

		Tree.UpdateElement(Index, Aabb, true);
	}

	void FLegacyAabbTree::Remove(FSpatialHandle& InOutHandle)
	{
		const int32 Index = (int32)InOutHandle.GetValue();
		check(Entries.IsValidIndex(Index));
		check(Entries[Index].bIsUsed);
		InOutHandle.SetValue(INDEX_NONE);

		Tree.RemoveElement(Index);
		Entries[Index].bIsUsed = false;
	}

	EVisitResult FLegacyAabbTree::Overlap(FOverlapQueryRuntimeData& QueryData, FOverlapVisitor& Visitor) const
	{
		FVisitorWrapper VisitorWrapper(this, &QueryData, &Visitor);
		TSpatialVisitor<int32, FReal> ProxyVisitor(VisitorWrapper);
		Tree.Overlap(QueryData.GetInputData().Aabb, ProxyVisitor);
		return EVisitResult::Continue;
	}

	EVisitResult FLegacyAabbTree::Raycast(FRaycastQueryRuntimeData& QueryData, FRaycastVisitor& Visitor) const
	{
		const FRaycastQueryData& InputData = QueryData.GetInputData();
		FVisitorWrapper VisitorWrapper(this, &QueryData, &Visitor);
		TSpatialVisitor<int32, FReal> ProxyVisitor(VisitorWrapper);
		Tree.Raycast(InputData.Start, InputData.Direction, QueryData.GetCurrentLength(), ProxyVisitor);
		return EVisitResult::Continue;
	}

	EVisitResult FLegacyAabbTree::Sweep(FSweepQueryRuntimeData& QueryData, FSweepVisitor& Visitor) const
	{
		const FSweepQueryData& InputData = QueryData.GetInputData();
		FVisitorWrapper VisitorWrapper(this, &QueryData, &Visitor);
		TSpatialVisitor<int32, FReal> ProxyVisitor(VisitorWrapper);
		Tree.Sweep(InputData.Start, InputData.Direction, QueryData.GetCurrentLength(), InputData.HalfExtents, ProxyVisitor);
		return EVisitResult::Continue;
	}

	void FLegacyAabbTree::SelfQuery(FSelfQueryVisitor& Visitor)
	{
		Tree.CacheOverlappingLeaves();

		for (int32 I = 0; I < Entries.Num(); ++I)
		{
			const FEntry& Entry = Entries[I];
			if (Entry.bIsUsed)
			{
				FSelfQueryVisitorWrapper VisitorWrapper(this, I, Entry.UserData, &Visitor);
				TSpatialVisitor<int32, FReal> ProxyVisitor(VisitorWrapper);
				Tree.Overlap(Entry.Aabb, ProxyVisitor);
			}
		}
	}

	FLegacyAabbTree::FVisitorWrapper::FVisitorWrapper(const FLegacyAabbTree* Tree, FOverlapQueryRuntimeData* InQueryData, FOverlapVisitor* InVisitor)
		: FLegacyVisitorBase(InQueryData, InVisitor)
		, Tree(Tree)
	{
	}

	FLegacyAabbTree::FVisitorWrapper::FVisitorWrapper(const FLegacyAabbTree* Tree, FRaycastQueryRuntimeData* InQueryData, FRaycastVisitor* InVisitor)
		: FLegacyVisitorBase(InQueryData, InVisitor)
		, Tree(Tree)
	{
	}

	FLegacyAabbTree::FVisitorWrapper::FVisitorWrapper(const FLegacyAabbTree* Tree, FSweepQueryRuntimeData* InQueryData, FSweepVisitor* InVisitor)
		: FLegacyVisitorBase(InQueryData, InVisitor)
		, Tree(Tree)
	{
	}

	bool FLegacyAabbTree::FVisitorWrapper::Overlap(const TSpatialVisitorData<int32>& Instance)
	{
		const FUserDataType UserData = Tree->Entries[Instance.Payload].UserData;
		return OverlapInternal(UserData);
	}

	bool FLegacyAabbTree::FVisitorWrapper::Raycast(const TSpatialVisitorData<int32>& Instance, FQueryFastData& CurData)
	{
		const FUserDataType UserData = Tree->Entries[Instance.Payload].UserData;
		return RaycastInternal(UserData, CurData);
	}

	bool FLegacyAabbTree::FVisitorWrapper::Sweep(const TSpatialVisitorData<int32>& Instance, FQueryFastData& CurData)
	{
		const FUserDataType UserData = Tree->Entries[Instance.Payload].UserData;
		return SweepInternal(UserData, CurData);
	}

	FLegacyAabbTree::FSelfQueryVisitorWrapper::FSelfQueryVisitorWrapper(const FLegacyAabbTree* Tree, const int32 Entry0Index, const FUserDataType& UserData0, FSelfQueryVisitor* InVisitor)
		: Visitor(InVisitor)
		, Tree(Tree)
		, Entry0Index(Entry0Index)
		, UserData0(UserData0)
	{
		check(Visitor != nullptr);
	}

	bool FLegacyAabbTree::FSelfQueryVisitorWrapper::Overlap(const TSpatialVisitorData<int32>& Instance)
	{
		// There's two cases of duplicates we can get:
		// 1. Entry0 == Entry1
		// 2. (Entry0, Entry1) && (Entry1, Entry0)
		// Both can be skipped by only keeping entries where 1 is explicitly larger than 0.
		const int32 Entry1Index = Instance.Payload;
		if (Entry1Index > Entry0Index)
		{
			const FUserDataType UserData1 = Tree->Entries[Entry1Index].UserData;
			Visitor->Visit(UserData0, UserData1);
		}

		return true;
	}

	bool FLegacyAabbTree::FSelfQueryVisitorWrapper::Raycast(const TSpatialVisitorData<int32>& Instance, FQueryFastData& CurData)
	{
		return false;
	}

	bool FLegacyAabbTree::FSelfQueryVisitorWrapper::Sweep(const TSpatialVisitorData<int32>& Instance, FQueryFastData& CurData)
	{
		return false;
	}

	const void* FLegacyAabbTree::FSelfQueryVisitorWrapper::GetQueryPayload() const
	{
		// This has to be overridden for the overlap cache to work
		return &UserData0;
	}
} // namespace Chaos::SpatialPartition
