// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "LegacyVisitorBase.h"
#include "ChaosSpatialPartitions/SpatialHandle.h"
#include "ChaosSpatialPartitions/Visitors.h"

#include "Chaos/AABBTree.h"

namespace Chaos::SpatialPartition
{
	// A context to cache certain cvars that we want to change during tests and then reset afterwards.
	struct FLegacyAabbTreeCVarCacheContext
	{
		FLegacyAabbTreeCVarCacheContext();
		~FLegacyAabbTreeCVarCacheContext();

		int32 DynamicTreeLeafCapacity;
		FReal DynamicTreeLeafEnlargePercent;
		FReal DynamicTreeBoundingBoxPadding;
	};

	class FLegacyAabbTree
	{
	public:
		struct FConfig
		{
			bool bDynamicTree = true;
			int32 MaxChildrenInLeaf = 8;
			int32 MaxTreeDepth = 200;
			FReal MaxPayloadBounds = 100000;
			int32 MaxNumToProcess = 4000;
		};

		FLegacyAabbTree() = default;
		FLegacyAabbTree(const FConfig& Config);

		void Insert(const FUserDataType& UserData, const FAABB3& Aabb, FSpatialHandle& OutHandle);
		void Update(const FUserDataType& UserData, const FAABB3& Aabb, FSpatialHandle& InOutHandle);
		void Remove(FSpatialHandle& InOutHandle);

		EVisitResult Overlap(FOverlapQueryRuntimeData& QueryData, FOverlapVisitor& Visitor) const;
		EVisitResult Raycast(FRaycastQueryRuntimeData& QueryData, FRaycastVisitor& Visitor) const;
		EVisitResult Sweep(FSweepQueryRuntimeData& QueryData, FSweepVisitor& Visitor) const;
		void SelfQuery(FSelfQueryVisitor& Visitor);

	private:
		// A wrapper for the legacy visitor to call into the new visitors.
		struct FVisitorWrapper : public FLegacyVisitorBase
		{
			FVisitorWrapper(const FLegacyAabbTree* Tree, FOverlapQueryRuntimeData* InQueryData, FOverlapVisitor* InVisitor);
			FVisitorWrapper(const FLegacyAabbTree* Tree, FRaycastQueryRuntimeData* InQueryData, FRaycastVisitor* InVisitor);
			FVisitorWrapper(const FLegacyAabbTree* Tree, FSweepQueryRuntimeData* InQueryData, FSweepVisitor* InVisitor);

			virtual bool Overlap(const TSpatialVisitorData<int32>& Instance) override;
			virtual bool Raycast(const TSpatialVisitorData<int32>& Instance, FQueryFastData& CurData) override;
			virtual bool Sweep(const TSpatialVisitorData<int32>& Instance, FQueryFastData& CurData) override;

			const FLegacyAabbTree* Tree = nullptr;
		};

		struct FSelfQueryVisitorWrapper : public ISpatialVisitor<int32>
		{
			FSelfQueryVisitorWrapper(const FLegacyAabbTree* Tree, const int32 Entry0Index, const FUserDataType& UserData0, FSelfQueryVisitor* InVisitor);
			virtual bool Overlap(const TSpatialVisitorData<int32>& Instance) override;
			virtual bool Raycast(const TSpatialVisitorData<int32>& Instance, FQueryFastData& CurData) override;
			virtual bool Sweep(const TSpatialVisitorData<int32>& Instance, FQueryFastData& CurData) override;

			virtual const void* GetQueryPayload() const override;

			FSelfQueryVisitor* Visitor = nullptr;
			const FLegacyAabbTree* Tree = nullptr;
			int32 Entry0Index;
			FUserDataType UserData0;
		};

		struct FEntry
		{
			FAABB3 Aabb;
			FUserDataType UserData;
			bool bIsUsed = false;
		};

		FConfig Config;
		FLegacyAabbTreeCVarCacheContext CVarCacheContext;
		TArray<FEntry> Entries;
		using AABBDynamicTreeType = TAABBTree<int, TAABBTreeLeafArray<int>>;
		AABBDynamicTreeType Tree;
	};
} // namespace Chaos::SpatialPartition
