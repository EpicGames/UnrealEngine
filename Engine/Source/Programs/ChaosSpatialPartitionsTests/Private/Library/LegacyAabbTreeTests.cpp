// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "ChaosTestHarness.h"

#include "TestVisitors.h"
#include "SharedOverlapTests.h"
#include "SharedRaycastTests.h"
#include "SharedSweepTests.h"
#include "SharedSelfQueryTests.h"

#include "LegacyAabbTree.h"

namespace Chaos::SpatialPartition
{
	TEST_CASE("LegacyDynamicAabbTree - Basic Queries", "[Chaos][LegacyDynamicAabbTree][spatial-partition]")
	{
		FLegacyAabbTree::FConfig Config{ .bDynamicTree = true, .MaxChildrenInLeaf = 8 };
		FLegacyAabbTree SpatialPartition(Config);
		SECTION("Overlap")
		{
			TestOverlapBuildBasic(SpatialPartition);
		}
		SECTION("Raycast")
		{
			TestRaycastBuildBasic(SpatialPartition);
		}
		SECTION("Sweep")
		{
			TestSweepBuildBasic(SpatialPartition);
		}
		SECTION("SelfQuery")
		{
			TestSelfQueryBuildBasic(SpatialPartition);
		}
	}

	TEST_CASE("LegacyDynamicAabbTree - Overlap - Performance", "[Chaos][LegacyDynamicAabbTree][spatial-partition][!benchmark]")
	{
		int32 MaxChildrenInLeaf = GENERATE(1, 8);
		FLegacyAabbTree::FConfig Config{ .bDynamicTree = true, .MaxChildrenInLeaf = MaxChildrenInLeaf };
		FLegacyAabbTree SpatialPartition(Config);
		SECTION("Overlap: Empty")
		{
			TestOverlapPerformanceEmpty(SpatialPartition);
		}
		SECTION("Overlap: Basic")
		{
			TestOverlapPerformanceBasic(SpatialPartition, 15, 15, 15);
		}
	}

	TEST_CASE("LegacyDynamicAabbTree - Raycast - Performance", "[Chaos][LegacyDynamicAabbTree][spatial-partition][!benchmark]")
	{
		int32 MaxChildrenInLeaf = GENERATE(1, 8);
		FLegacyAabbTree::FConfig Config{ .bDynamicTree = true, .MaxChildrenInLeaf = MaxChildrenInLeaf };
		FLegacyAabbTree SpatialPartition(Config);
		SECTION("Raycast: Empty")
		{
			TestRaycastPerformanceEmpty(SpatialPartition);
		}
		SECTION("Raycast: Basic")
		{
			TestRaycastPerformanceBasic(SpatialPartition);
		}
		SECTION("Raycast: First Hit")
		{
			TestRaycastPerformanceFirstHit(SpatialPartition);
		}
	}

	TEST_CASE("LegacyDynamicAabbTree - Sweep - Performance", "[Chaos][LegacyDynamicAabbTree][spatial-partition][!benchmark]")
	{
		int32 MaxChildrenInLeaf = GENERATE(1, 8);
		FLegacyAabbTree::FConfig Config{ .bDynamicTree = true, .MaxChildrenInLeaf = MaxChildrenInLeaf };
		FLegacyAabbTree SpatialPartition(Config);
		SECTION("Sweep: Empty")
		{
			TestSweepPerformanceEmpty(SpatialPartition);
		}
		SECTION("Sweep: Basic")
		{
			TestSweepPerformanceBasic(SpatialPartition);
		}
		SECTION("Sweep: First Hit")
		{
			TestSweepPerformanceFirstHit(SpatialPartition);
		}
	}

	TEST_CASE("LegacyDynamicAabbTree - SelfQuery - Performance", "[Chaos][LegacyDynamicAabbTree][spatial-partition][!benchmark]")
	{
		int32 MaxChildrenInLeaf = GENERATE(1, 8);
		FLegacyAabbTree::FConfig Config{ .bDynamicTree = true, .MaxChildrenInLeaf = MaxChildrenInLeaf };
		FLegacyAabbTree SpatialPartition(Config);
		SECTION("Self Query: Empty")
		{
			TestSelfQueryPerformanceEmpty(SpatialPartition);
		}
		SECTION("Self Query: Basic")
		{
			TestSelfQueryPerformanceBasic(SpatialPartition, 15, 15, 15);
		}
	}
} // namespace Chaos::SpatialPartition
