// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "ChaosTestHarness.h"

#include "TestVisitors.h"
#include "SharedOverlapTests.h"
#include "SharedRaycastTests.h"
#include "SharedSweepTests.h"
#include "SharedSelfQueryTests.h"

#include "ChaosSpatialPartitions/Library/NSquaredAabb.h"

namespace Chaos::SpatialPartition
{
	TEST_CASE("NSquaredAabb - Overlap - Basic", "[Chaos][NSquaredAabb][spatial-partition]")
	{
		FNSquaredAabb SpatialPartition;
		SECTION("Build")
		{
			TestOverlapBuildBasic(SpatialPartition);
		}
		SECTION("Insert")
		{
			TestOverlapInsert(SpatialPartition);
		}
		SECTION("Remove")
		{
			TestOverlapRemove(SpatialPartition);
		}
		SECTION("Update: Basic")
		{
			TestOverlapUpdateBasic(SpatialPartition);
		}
		SECTION("Update: Within Bounds")
		{
			TestOverlapUpdateWithinBounds(SpatialPartition);
		}
		SECTION("Update: VisitResult Early Termination")
		{
			TestOverlapWithEarlyVisitResultTermination(SpatialPartition);
		}
	}

	TEST_CASE("NSquaredAabb - Overlap - Performance", "[Chaos][NSquaredAabb][spatial-partition][!benchmark]")
	{
		FNSquaredAabb SpatialPartition;
		SECTION("Overlap: Empty")
		{
			TestOverlapPerformanceEmpty(SpatialPartition);
		}
		SECTION("Overlap: Basic")
		{
			TestOverlapPerformanceBasic(SpatialPartition, 15, 15, 15);
		}
	}

	TEST_CASE("NSquaredAabb - Raycast - Basic", "[Chaos][NSquaredAabb][spatial-partition]")
	{
		FNSquaredAabb SpatialPartition;
		SECTION("Build")
		{
			TestRaycastBuildBasic(SpatialPartition);
		}
		SECTION("Insert")
		{
			TestRaycastInsert(SpatialPartition);
		}
		SECTION("Remove")
		{
			TestRaycastRemove(SpatialPartition);
		}
		SECTION("Update: Basic")
		{
			TestRaycastUpdateBasic(SpatialPartition);
		}
		SECTION("Update: Within Bounds")
		{
			TestRaycastUpdateWithinBounds(SpatialPartition);
		}
		SECTION("Update: VisitResult Early Termination")
		{
			TestRaycastWithEarlyVisitResultTermination(SpatialPartition);
		}
		SECTION("Update: Length Early Termination")
		{
			TestRaycastWithEarlyLengthTermination(SpatialPartition);
		}
	}

	TEST_CASE("NSquaredAabb - Raycast - Performance", "[Chaos][NSquaredAabb][spatial-partition][!benchmark]")
	{
		FNSquaredAabb SpatialPartition;
		SECTION("Raycast: Empty")
		{
			TestRaycastPerformanceEmpty(SpatialPartition);
		}
		SECTION("Raycast: Basic")
		{
			TestRaycastPerformanceBasic(SpatialPartition);
		}
		SECTION("Raycast: First Hit")
		{
			TestRaycastPerformanceFirstHit(SpatialPartition);
		}
	}

	TEST_CASE("NSquaredAabb - Sweep - Basic", "[Chaos][NSquaredAabb][spatial-partition]")
	{
		FNSquaredAabb SpatialPartition;
		SECTION("Build")
		{
			TestSweepBuildBasic(SpatialPartition);
		}
		SECTION("Insert")
		{
			TestSweepInsert(SpatialPartition);
		}
		SECTION("Remove")
		{
			TestSweepRemove(SpatialPartition);
		}
		SECTION("Update: Basic")
		{
			TestSweepUpdateBasic(SpatialPartition);
		}
		SECTION("Update: Within Bounds")
		{
			TestSweepUpdateWithinBounds(SpatialPartition);
		}
		SECTION("Update: VisitResult Early Termination")
		{
			TestSweepWithEarlyVisitResultTermination(SpatialPartition);
		}
		SECTION("Update: Length Early Termination")
		{
			TestSweepWithEarlyLengthTermination(SpatialPartition);
		}
	}

	TEST_CASE("NSquaredAabb - Sweep - Performance", "[Chaos][NSquaredAabb][spatial-partition][!benchmark]")
	{
		FNSquaredAabb SpatialPartition;
		SECTION("Sweep: Empty")
		{
			TestSweepPerformanceEmpty(SpatialPartition);
		}
		SECTION("Sweep: Basic")
		{
			TestSweepPerformanceBasic(SpatialPartition);
		}
		SECTION("Sweep: First Hit")
		{
			TestSweepPerformanceFirstHit(SpatialPartition);
		}
	}

	TEST_CASE("NSquaredAabb - SelfQuery - Basic", "[Chaos][NSquaredAabb][spatial-partition]")
	{
		FNSquaredAabb SpatialPartition;
		SECTION("Build")
		{
			TestSelfQueryBuildBasic(SpatialPartition);
		}
		SECTION("Insert")
		{
			TestSelfQueryInsert(SpatialPartition);
		}
		SECTION("Remove")
		{
			TestSelfQueryRemove(SpatialPartition);
		}
		SECTION("Update: Basic")
		{
			TestSelfQueryUpdateBasic(SpatialPartition);
		}
		SECTION("Update: Movement")
		{
			TestSelfQueryUpdateMovement(SpatialPartition);
		}
		SECTION("Update: Within Bounds")
		{
			TestSelfQueryUpdateWithinBounds(SpatialPartition);
		}
	}

	TEST_CASE("NSquaredAabb - SelfQuery - Performance", "[Chaos][NSquaredAabb][spatial-partition][!benchmark]")
	{
		FNSquaredAabb SpatialPartition;
		SECTION("Self Query: Empty")
		{
			TestSelfQueryPerformanceEmpty(SpatialPartition);
		}
		SECTION("Self Query: Basic")
		{
			TestSelfQueryPerformanceBasic(SpatialPartition, 15, 15, 15);
		}
	}
} // namespace Chaos::SpatialPartition
