// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "ChaosTestHarness.h"

#include "AabbTreeTestHelpers.h"
#include "TestVisitors.h"
#include "SharedOverlapTests.h"
#include "SharedRaycastTests.h"
#include "SharedSweepTests.h"
#include "SharedSelfQueryTests.h"
#include "StaticAabbTreeHelpers.h"

#include "ChaosSpatialPartitions/Library/StaticAabbTree.h"

namespace Chaos::SpatialPartition
{
	// Wrapper around the tree to make testing easier. 
	// Provides some common functionality and (eventual) flags to more easily parameterize tests.
	class FTestStaticAabbTree : public FStaticAabbTree
	{
	public:
		using FStaticAabbTree::FStaticAabbTree;
		using FStaticAabbTree::InsertDeferred;
		using FStaticAabbTree::UpdateDeferred;
		using FStaticAabbTree::RemoveMinimal;

		void InsertDeferred(FObjectData& Object)
		{
			InsertDeferred(Object.UserData, Object.Aabb, Object.Handle);
		}

		void UpdateDeferred(FObjectData& Object)
		{
			UpdateDeferred(Object.UserData, Object.Aabb, Object.Handle);
		}

		void RemoveMinimal(FObjectData& Object)
		{
			RemoveMinimal(Object.Handle);
		}

		void Rebuild()
		{
			FTestStaticAabbTree::FRebuildContext Context;
			BeginRebuild(Context);
			while (ISpatialPartition::ERebuildStatus::Continue == Context.Run())
			{

			}
			CommitRebuild(Context);
		}

		// Runs a callback while the rebuild is in progress. This is to test what happens for mutations during a rebuild.
		template <typename CallbackType>
		void RebuildWithCallbackBeforeCommit(CallbackType Callback)
		{
			FTestStaticAabbTree::FRebuildContext Context;
			BeginRebuild(Context);
			Callback();
			while (ISpatialPartition::ERebuildStatus::Continue == Context.Run())
			{

			}
			CommitRebuild(Context);
		}
	};

	template <>
	void InsertObject(FTestStaticAabbTree& SpatialPartition, FObjectData& Object)
	{
		SpatialPartition.InsertDeferred(Object);
		SpatialPartition.Rebuild();
	}

	template <>
	void UpdateObject(FTestStaticAabbTree& SpatialPartition, FObjectData& Object)
	{
		SpatialPartition.UpdateDeferred(Object);
		SpatialPartition.Rebuild();
	}

	template <>
	void RemoveObject(FTestStaticAabbTree& SpatialPartition, FObjectData& Object)
	{
		SpatialPartition.RemoveMinimal(Object);
		SpatialPartition.Rebuild();
	}

	template <>
	void BuildFromObjects(FTestStaticAabbTree& SpatialPartition, TArray<FObjectData>& Objects)
	{
		for (FObjectData& Object : Objects)
		{
			SpatialPartition.InsertDeferred(Object);
		}
		SpatialPartition.Rebuild();
	}

	TEST_CASE("StaticAabbTree - Overlap - Basic", "[Chaos][StaticAabbTree][spatial-partition]")
	{
		FTestStaticAabbTree SpatialPartition;
		SECTION("Build")
		{
			TestOverlapBuildBasic(SpatialPartition);
		}
		SECTION("Update: Within Bounds")
		{
			TestOverlapUpdateWithinBounds(SpatialPartition);
		}
		SECTION("Update: VisitResult Early Termination")
		{
			TestOverlapWithEarlyVisitResultTermination(SpatialPartition);
		}
	}

	TEST_CASE("StaticAabbTree - Overlap - Performance", "[Chaos][StaticAabbTree][spatial-partition][!benchmark]")
	{
		FTestStaticAabbTree SpatialPartition;
		SECTION("Overlap: Empty")
		{
			TestOverlapPerformanceEmpty(SpatialPartition);
		}
		SECTION("Overlap: Basic")
		{
			TestOverlapPerformanceBasic(SpatialPartition, 15, 15, 15);
		}
	}

	TEST_CASE("StaticAabbTree - Raycast - Basic", "[Chaos][StaticAabbTree][spatial-partition]")
	{
		FTestStaticAabbTree SpatialPartition;
		SECTION("Build")
		{
			TestRaycastBuildBasic(SpatialPartition);
		}
		SECTION("Update: VisitResult Early Termination")
		{
			TestRaycastWithEarlyVisitResultTermination(SpatialPartition);
		}
		SECTION("Update: Length Early Termination")
		{
			TestRaycastWithEarlyLengthTermination(SpatialPartition);
		}
	}

	TEST_CASE("StaticAabbTree - Raycast - Performance", "[Chaos][StaticAabbTree][spatial-partition][!benchmark]")
	{
		FTestStaticAabbTree SpatialPartition;
		SECTION("Raycast: Empty")
		{
			TestRaycastPerformanceEmpty(SpatialPartition);
		}
		SECTION("Raycast: Basic")
		{
			TestRaycastPerformanceBasic(SpatialPartition);
		}
		SECTION("Raycast: First Hit")
		{
			TestRaycastPerformanceFirstHit(SpatialPartition);
		}
	}

	TEST_CASE("StaticAabbTree - Sweep - Basic", "[Chaos][StaticAabbTree][spatial-partition]")
	{
		FTestStaticAabbTree SpatialPartition;
		SECTION("Build")
		{
			TestSweepBuildBasic(SpatialPartition);
		}
		SECTION("Update: VisitResult Early Termination")
		{
			TestSweepWithEarlyVisitResultTermination(SpatialPartition);
		}
		SECTION("Update: Length Early Termination")
		{
			TestSweepWithEarlyLengthTermination(SpatialPartition);
		}
	}

	TEST_CASE("StaticAabbTree - Sweep - Performance", "[Chaos][StaticAabbTree][spatial-partition][!benchmark]")
	{
		FTestStaticAabbTree SpatialPartition;
		SECTION("Sweep: Empty")
		{
			TestSweepPerformanceEmpty(SpatialPartition);
		}
		SECTION("Sweep: Basic")
		{
			TestSweepPerformanceBasic(SpatialPartition);
		}
		SECTION("Sweep: First Hit")
		{
			TestSweepPerformanceFirstHit(SpatialPartition);
		}
	}

	TEST_CASE("StaticAabbTree - InsertDeferred", "[Chaos][StaticAabbTree][spatial-partition]")
	{
		FTestStaticAabbTree::FConfig Config;
		Config.BatchSize = 1000;
		Config.MaxElementsPerLeaf = 1;
		Config.PartitioningMethod = EPartitioningMethod::CentroidSpatialMedian;
		FTestStaticAabbTree SpatialPartition(Config);

		TArray<FObjectData> Objects;
		BuildObjectList(Objects, 3);

		TArray<FAabbTreeNode> ActualNodes;
		TArray<FAabbTreeLeaf> ActualLeaves;
		int32 ActualRootIndex;
		TArray<const FAabbTreeLeaf*> Leaves;
		TArray<const FAabbTreeNode*> Nodes;

		SECTION("Empty Tree")
		{
			SECTION("InsertDeferred without Build")
			{
				for (int32 I = 0; I < 2; ++I)
				{
					SpatialPartition.InsertDeferred(Objects[I]);
					SpatialPartition.Dump(ActualNodes, ActualRootIndex, ActualLeaves);
					CHECK(ActualRootIndex == INDEX_NONE);
				}
			}
			SECTION("InsertDeferred with Build")
			{
				// Insert and build with 2 objects. This should produce a simple split tree.
				for (int32 I = 0; I < 2; ++I)
				{
					SpatialPartition.InsertDeferred(Objects[I]);
				}
				SpatialPartition.Rebuild();

				SpatialPartition.Dump(ActualNodes, ActualRootIndex, ActualLeaves);
				FindLeafAndNodePointers(ActualLeaves, ActualNodes, 2, Leaves, Nodes);

				// Validate we found a leaf and node for each object and their aabbs are expected.
				for (int32 I = 0; I < 2; ++I)
				{
					REQUIRE(Leaves[I] != nullptr);
					REQUIRE(Nodes[I] != nullptr);

					CHECK(Leaves[I]->Elements.Num() == 1);
					CHECK_THAT(Leaves[I]->Aabb, Catch::Equal(Objects[I].Aabb));
					CHECK_THAT(Nodes[I]->Aabb, Catch::Equal(Objects[I].Aabb));
				}

				// Validate the tree structure/aabbs.
				const FAABB3 Aabb01 = Union(Objects[0].Aabb, Objects[1].Aabb);
				// Check the root
				CHECK(Nodes[0]->Parent == Nodes[1]->Parent);
				CHECK(Nodes[0]->Parent == ActualRootIndex);
				CHECK_THAT(ActualNodes[ActualRootIndex].Aabb, Catch::Equal(Aabb01));
			}
		}
		SECTION("Non-Empty Tree")
		{
			for (int32 I = 0; I < 2; ++I)
			{
				SpatialPartition.InsertDeferred(Objects[I]);
			}
			SpatialPartition.Rebuild();

			SpatialPartition.InsertDeferred(Objects[2]);
			SECTION("InsertDeferred without Build")
			{
				SpatialPartition.Dump(ActualNodes, ActualRootIndex, ActualLeaves);
				FindLeafAndNodePointers(ActualLeaves, ActualNodes, 3, Leaves, Nodes);

				// Validate we found a leaf and node for all built objects.
				for (int32 I = 0; I < 2; ++I)
				{
					REQUIRE(Leaves[I] != nullptr);
					REQUIRE(Nodes[I] != nullptr);

					CHECK(Leaves[I]->Elements.Num() == 1);
					CHECK_THAT(Leaves[I]->Aabb, Catch::Equal(Objects[I].Aabb));
					CHECK_THAT(Nodes[I]->Aabb, Catch::Equal(Objects[I].Aabb));
				}
				// We don't expect the last deferred insert to have data yet.
				CHECK(Leaves[2] == nullptr);
			}
			SECTION("InsertDeferred with Build")
			{
				SpatialPartition.Rebuild();

				SpatialPartition.Dump(ActualNodes, ActualRootIndex, ActualLeaves);
				FindLeafAndNodePointers(ActualLeaves, ActualNodes, 3, Leaves, Nodes);
				// Validate we found a leaf and node for each object and their aabbs are expected.
				for (int32 I = 0; I < 3; ++I)
				{
					REQUIRE(Leaves[I] != nullptr);
					REQUIRE(Nodes[I] != nullptr);

					CHECK(Leaves[I]->Elements.Num() == 1);
					CHECK_THAT(Leaves[I]->Aabb, Catch::Equal(Objects[I].Aabb));
					CHECK_THAT(Nodes[I]->Aabb, Catch::Equal(Objects[I].Aabb));
				}
			}
		}
	}

	TEST_CASE("StaticAabbTree - UpdateDeferred", "[Chaos][StaticAabbTree][spatial-partition]")
	{
		TArray<FObjectData> Objects;
		BuildObjectList(Objects, 2);

		int32 ActualRootIndex;
		TArray<FAabbTreeNode> ActualNodes;
		TArray<FAabbTreeLeaf> ActualLeaves;
		TArray<const FAabbTreeNode*> Nodes;
		TArray<const FAabbTreeLeaf*> Leaves;

		// Setup the default scenario
		FTestStaticAabbTree SpatialPartition(FTestStaticAabbTree::FConfig{ .MaxElementsPerLeaf = 1 });
		BuildFromObjects(SpatialPartition, Objects);

		// Now cache the old aabbs and update both to new values.
		const FAABB3 OldAabbs[2]
		{
			Objects[0].Aabb,
			Objects[1].Aabb,
		};
		// Update 0 so the new aabb is within the bounds of the old and 1 so it's outside the original bounds
		Objects[0].Aabb.ShrinkSymmetrically(FVec3(0.1f));
		Objects[1].Aabb.MoveByVector(FVec3(5));

		// Now update all the objects and verify nothing has changed
		for (FObjectData& Object : Objects)
		{
			SpatialPartition.UpdateDeferred(Object);
		}

		SpatialPartition.Dump(ActualNodes, ActualRootIndex, ActualLeaves);
		FindLeafAndNodePointers(ActualLeaves, ActualNodes, 2, Leaves, Nodes);
		for (int32 I = 0; I < 2; ++I)
		{
			REQUIRE(Leaves[I]->Elements.Num() == 1);
			CHECK_THAT(Leaves[I]->Elements[0].Aabb, Catch::Equal(OldAabbs[I]));
			CHECK_THAT(Leaves[I]->Aabb, Catch::Equal(OldAabbs[I]));
			CHECK_THAT(Nodes[I]->Aabb, Catch::Equal(OldAabbs[I]));
		}

		// Now call build which should flush the changes
		SpatialPartition.Rebuild();

		SpatialPartition.Dump(ActualNodes, ActualRootIndex, ActualLeaves);
		FindLeafAndNodePointers(ActualLeaves, ActualNodes, 2, Leaves, Nodes);
		for (int32 I = 0; I < 2; ++I)
		{
			REQUIRE(Leaves[I]->Elements.Num() == 1);
			CHECK_THAT(Leaves[I]->Elements[0].Aabb, Catch::Equal(Objects[I].Aabb));
			CHECK_THAT(Leaves[I]->Aabb, Catch::Equal(Objects[I].Aabb));
			CHECK_THAT(Nodes[I]->Aabb, Catch::Equal(Objects[I].Aabb));
		}
	}

	TEST_CASE("StaticAabbTree - RemoveMinimal", "[Chaos][StaticAabbTree][spatial-partition]")
	{
		const FVec3 AabbSize(1);
		TArray<FObjectData> Objects
		{
			FObjectData{.Aabb = BuildAabbCenterExtents(FVec3(0), AabbSize), .UserData = 0},
			FObjectData{.Aabb = BuildAabbCenterExtents(FVec3(1), AabbSize), .UserData = 1},
			FObjectData{.Aabb = BuildAabbCenterExtents(FVec3(4), AabbSize), .UserData = 2},
		};
		const FAABB3 Aabb01 = Union(Objects[0].Aabb, Objects[1].Aabb);
		const FAABB3 Aabb012 = Union(Aabb01, Objects[2].Aabb);

		int32 ActualRootIndex;
		TArray<FAabbTreeNode> ActualNodes;
		TArray<FAabbTreeLeaf> ActualLeaves;
		TArray<const FAabbTreeNode*> Nodes;
		TArray<const FAabbTreeLeaf*> Leaves;

		// Setup the default scenario
		FTestStaticAabbTree SpatialPartition(FTestStaticAabbTree::FConfig{ .MaxElementsPerLeaf = 2 });
		BuildFromObjects(SpatialPartition, Objects);

		SECTION("Remove results in non-empty leaf")
		{
			SpatialPartition.RemoveMinimal(Objects[0]);

			SpatialPartition.Dump(ActualNodes, ActualRootIndex, ActualLeaves);
			FindLeafAndNodePointers(ActualLeaves, ActualNodes, Objects.Num(), Leaves, Nodes);

			CHECK(Leaves[0] == nullptr);
			REQUIRE(Leaves[1] != nullptr);
			REQUIRE(Leaves[2] != nullptr);

			// Check leaf 01
			CHECK(Leaves[1]->Elements.Num() == 1);
			CHECK(Leaves[1]->Elements[0].Index == Objects[1].UserData);
			CHECK_THAT(Leaves[1]->Elements[0].Aabb, Catch::Equal(Objects[1].Aabb));
			CHECK_THAT(Leaves[1]->Aabb, Catch::Equal(Aabb01));
			CHECK_THAT(Nodes[1]->Aabb, Catch::Equal(Aabb01));
			// Check leaf 2
			CHECK_THAT(Leaves[2]->Aabb, Catch::Equal(Objects[2].Aabb));
			CHECK_THAT(Nodes[2]->Aabb, Catch::Equal(Objects[2].Aabb));
			// Check the root
			CHECK_THAT(ActualNodes[ActualRootIndex].Aabb, Catch::Equal(Aabb012));
		}
		SECTION("Remove results in empty leaf")
		{
			SpatialPartition.RemoveMinimal(Objects[2]);

			SpatialPartition.Dump(ActualNodes, ActualRootIndex, ActualLeaves);
			FindLeafAndNodePointers(ActualLeaves, ActualNodes, Objects.Num(), Leaves, Nodes);

			REQUIRE(Leaves[0] != nullptr);
			REQUIRE(Leaves[1] != nullptr);
			CHECK(Leaves[0] == Leaves[1]);
			CHECK(Leaves[2] == nullptr);

			// Check leaf 01
			CHECK(Leaves[0]->Elements.Num() == 2);
			CHECK_THAT(Leaves[0]->Aabb, Catch::Equal(Aabb01));
			CHECK_THAT(Nodes[0]->Aabb, Catch::Equal(Aabb01));
			// Check the old leaf node 2
			const FAabbTreeNode& RootNode = ActualNodes[ActualRootIndex];
			const int32 Node2Index = (RootNode.Left == Leaves[0]->NodeIndex) ? RootNode.Right : RootNode.Left;
			const FAabbTreeNode& Node2 = ActualNodes[Node2Index];
			const FAabbTreeLeaf& Leaf2 = ActualLeaves[Node2.UserData];
			CHECK(Leaf2.Elements.IsEmpty());
			CHECK_THAT(Leaf2.Aabb, Catch::Equal(Objects[2].Aabb));
			CHECK_THAT(Node2.Aabb, Catch::Equal(Objects[2].Aabb));
			// Check the root
			CHECK_THAT(RootNode.Aabb, Catch::Equal(Aabb012));
		}
		SECTION("Remove then build")
		{
			SpatialPartition.RemoveMinimal(Objects[1]);
			SpatialPartition.RemoveMinimal(Objects[2]);
			SpatialPartition.Rebuild();

			SpatialPartition.Dump(ActualNodes, ActualRootIndex, ActualLeaves);
			FindLeafAndNodePointers(ActualLeaves, ActualNodes, Objects.Num(), Leaves, Nodes);

			REQUIRE(Leaves[0] != nullptr);
			CHECK(Leaves[1] == nullptr);
			CHECK(Leaves[2] == nullptr);

			CHECK_THAT(Leaves[0]->Aabb, Catch::Equal(Objects[0].Aabb));
			CHECK_THAT(Nodes[0]->Aabb, Catch::Equal(Objects[0].Aabb));
			CHECK(Leaves[0]->NodeIndex == ActualRootIndex);
		}
		SECTION("Remove after insert deferred without rebuild")
		{
			FObjectData Object4{ .Aabb = BuildAabbCenterExtents(FVec3(10), AabbSize), .UserData = 3 };
			SpatialPartition.InsertDeferred(Object4);
			SpatialPartition.RemoveMinimal(Object4);
		}
		SECTION("Basic query validation")
		{
			// Just do a basic overlap query to ensure that we don't return the removed objects somehow
			SpatialPartition.RemoveMinimal(Objects[1]);
			SpatialPartition.RemoveMinimal(Objects[2]);

			FOverlapQueryRuntimeData QueryRuntimeData(Aabb012);
			FTestOverlapCollectorVisitor Visitor;
			SpatialPartition.Overlap(QueryRuntimeData, Visitor);

			TArray<FUserDataType> ExpectedResults{ Objects[0].UserData };
			CHECK(Visitor.Results == ExpectedResults);
		}
	}

	TEST_CASE("StaticAabbTree - UpdateIfWithinLeaf", "[Chaos][StaticAabbTree][spatial-partition]")
	{
		const FVec3 AabbSize(1);
		TArray<FObjectData> Objects
		{
			FObjectData{.Aabb = BuildAabbCenterExtents(FVec3(0), AabbSize), .UserData = 0},
			FObjectData{.Aabb = BuildAabbCenterExtents(FVec3(4), AabbSize), .UserData = 1},
		};
		const FAABB3 RootAabb = Union(Objects[0].Aabb, Objects[1].Aabb);

		FObjectData& TestObject = Objects[0];
		FTestStaticAabbTree SpatialPartition(FTestStaticAabbTree::FConfig{ .MaxElementsPerLeaf = 1 });
		BuildFromObjects(SpatialPartition, Objects);

		SECTION("New Aabb Within Leaf")
		{
			FAABB3 NewAabb = TestObject.Aabb;
			NewAabb.ShrinkSymmetrically(FVec3(0.1));
			CHECK(SpatialPartition.UpdateIfWithinLeaf(TestObject.UserData, NewAabb, TestObject.Handle));
		}
		SECTION("New Aabb Outside Leaf")
		{
			FAABB3 NewAabb = TestObject.Aabb;
			NewAabb.MoveByVector(FVec3(1));
			CHECK(!SpatialPartition.UpdateIfWithinLeaf(TestObject.UserData, NewAabb, TestObject.Handle));
		}
		SECTION("Object deferred inserted without rebuild")
		{
			FObjectData Object2{ .Aabb = BuildAabbCenterExtents(FVec3(10), AabbSize), .UserData = 2 };
			SpatialPartition.InsertDeferred(Object2);
			CHECK(!SpatialPartition.UpdateIfWithinLeaf(Object2.UserData, Object2.Aabb, Object2.Handle));
		}
	}

	TEST_CASE("StaticAabbTree - Mutations During Rebuild", "[Chaos][StaticAabbTree][spatial-partition]")
	{
		const FTestStaticAabbTree::FConfig Config
		{
			.BatchSize = 1,
			.MaxElementsPerLeaf = 1,
		};
		FTestStaticAabbTree SpatialPartition(Config);

		TArray<FObjectData> Objects;

		BuildObjectList(Objects, 20);
		// Build the spatial partition with half of the data
		for (int32 I = 0; I < 10; ++I)
		{
			SpatialPartition.InsertDeferred(Objects[I]);
		}
		SpatialPartition.Rebuild();

		FTestOverlapCollectorVisitor Visitor;
		FTestStaticAabbTree::FRebuildContext Context;
		SECTION("Insert")
		{
			FObjectData& TargetObject = Objects[10];

			SpatialPartition.RebuildWithCallbackBeforeCommit([&TargetObject, &SpatialPartition]
			{
				SpatialPartition.InsertDeferred(TargetObject);
			});

			FOverlapQueryRuntimeData QueryData(TargetObject.Aabb);
			SpatialPartition.Overlap(QueryData, Visitor);
			CHECK(Visitor.Results.IsEmpty());
		}
		SECTION("Update")
		{
			FObjectData& TargetObject = Objects[1];
			const FAABB3 OldAabb1 = TargetObject.Aabb;
			TargetObject.Aabb = FAABB3(FVector(100), FVector(101));

			SpatialPartition.RebuildWithCallbackBeforeCommit([&TargetObject, &SpatialPartition]
			{
				SpatialPartition.UpdateDeferred(TargetObject);
			});

			FOverlapQueryRuntimeData QueryData0(OldAabb1);
			SpatialPartition.Overlap(QueryData0, Visitor);
			CHECK(Visitor.Results == TArray<FUserDataType> {TargetObject.UserData});
			Visitor.Reset();

			FOverlapQueryRuntimeData QueryData1(TargetObject.Aabb);
			SpatialPartition.Overlap(QueryData1, Visitor);
			CHECK(Visitor.Results.IsEmpty());
		}
		SECTION("Remove")
		{
			FObjectData& TargetObject = Objects[0];

			SpatialPartition.RebuildWithCallbackBeforeCommit([&TargetObject, &SpatialPartition]
			{
				SpatialPartition.RemoveMinimal(TargetObject);
			});

			FOverlapQueryRuntimeData QueryData(TargetObject.Aabb);
			SpatialPartition.Overlap(QueryData, Visitor);
			CHECK(Visitor.Results.IsEmpty());
		}
		SECTION("Concurrent Rebuilds")
		{
			FObjectData& TargetObject = Objects[11];

			// Tests what happens when multiple builds happen. The last commit should win.
			FTestStaticAabbTree::FRebuildContext Context1;
			FTestStaticAabbTree::FRebuildContext Context2;

			// Do a mutation in between the two rebuilds.
			SpatialPartition.BeginRebuild(Context1);
			SpatialPartition.InsertDeferred(TargetObject);
			SpatialPartition.BeginRebuild(Context2);

			while (ISpatialPartition::ERebuildStatus::Continue == Context1.Run())
			{

			}
			SpatialPartition.CommitRebuild(Context1);

			while (ISpatialPartition::ERebuildStatus::Continue == Context2.Run())
			{

			}
			SpatialPartition.CommitRebuild(Context2);

			FOverlapQueryRuntimeData QueryData(TargetObject.Aabb);
			SpatialPartition.Overlap(QueryData, Visitor);
			CHECK(Visitor.Results == TArray<FUserDataType> {TargetObject.UserData});
		}
	}
} // namespace Chaos::SpatialPartition
