// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "ChaosSpatialPartitions/SpatialClassification.h"
#include "ChaosSpatialPartitions/SpatialHandle.h"
#include "SharedHelpers.h"

namespace Chaos::SpatialPartition
{
	// Helper class to represent an object. This just makes it a bit easier to manage groups of objects for tests.
	struct FObjectData
	{
		FVec3 GetCenter() const { return Aabb.Center(); }
		const FAABB3& GetAabb() const { return Aabb; }

		FAABB3 Aabb = FAABB3::EmptyAABB();
		FUserDataType UserData = INDEX_NONE;
		FSpatialClassification Classification;
		FSpatialHandle Handle;
	};

	inline void BuildObjectList(TArray<FObjectData>& Objects, const int32 CountX = 1, const int32 CountY = 1, const int32 CountZ = 1, const FVec3& Spacing = FVec3(1), const FVec3& AabbSize = FVec3(1))
	{
		const FVec3 ScaleFactors = AabbSize + Spacing;
		Objects.SetNum(CountX * CountY * CountZ);
		for (int32 Z = 0; Z < CountZ; ++Z)
		{
			for (int32 Y = 0; Y < CountY; ++Y)
			{
				for (int32 X = 0; X < CountX; ++X)
				{
					const int32 I = GetIndex(X, Y, Z, CountX, CountY, CountZ);

					const FVec3 Min = FVec3(X, Y, Z) * ScaleFactors;
					Objects[I].Aabb = FAABB3(Min, Min + AabbSize);
					Objects[I].UserData = I;
				}
			}
		}
	}

	// These methods allow generic tests to update the structure of a spatial partition while
	// allowing specializations to change this behavior if needed (e.g. a static structure that needs a build step).

	template <typename SpatialPartitionType>
	void InsertObject(SpatialPartitionType& SpatialPartition, FObjectData& Object)
	{
		SpatialPartition.Insert(Object.UserData, Object.Aabb, Object.Handle);
	}

	template <typename SpatialPartitionType>
	void UpdateObject(SpatialPartitionType& SpatialPartition, FObjectData& Object)
	{
		SpatialPartition.Update(Object.UserData, Object.Aabb, Object.Handle);
	}

	template <typename SpatialPartitionType>
	void RemoveObject(SpatialPartitionType& SpatialPartition, FObjectData& Object)
	{
		SpatialPartition.Remove(Object.Handle);
	}

	template <typename SpatialPartitionType>
	void BuildFromObjects(SpatialPartitionType& SpatialPartition, TArray<FObjectData>& Objects)
	{
		for (FObjectData& Object : Objects)
		{
			SpatialPartition.Insert(Object.UserData, Object.Aabb, Object.Handle);
		}
	}
	
	// Tell the spatial partition to fully rebuild itself (no dirty state)
	template <typename SpatialPartitionType>
	void Rebuild(SpatialPartitionType& SpatialPartition)
	{
	}
} // namespace Chaos::SpatialPartition
