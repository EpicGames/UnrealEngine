// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "ChaosTestHarness.h"

#include "ChaosSpatialPartitions/SpatialHandle.h"

#include "TestVisitors.h"
#include "ObjectData.h"
#include "SharedHelpers.h"
#include "catch2/generators/catch_generators_range.hpp"

namespace Chaos::SpatialPartition
{
	inline void BuildObjectList(TArray<FObjectData>& Objects, const int32 ObjectCount, const TArray<FSpatialClassification>& Classifications, const FVec3& AabbSize = FVec3(1))
	{
		const int32 ClassificationCount = Classifications.Num();
		REQUIRE(ClassificationCount != 0);
		for (int32 I = 0; I < ObjectCount; ++I)
		{
			FObjectData Object
			{
				.Aabb = BuildAabbCenterExtents(FVec3(I, 0, 0), AabbSize),
				.UserData = I,
				.Classification = Classifications[I % ClassificationCount],
			};
			Objects.Emplace(Object);
		}
	}


	inline void BuildExpectedResults(const TArray<FObjectData>& Objects, const int32 IndexToSkip, TArray<FUserDataType>& OutResults)
	{
		for (int32 I = 0; I < Objects.Num(); ++I)
		{
			if (I != IndexToSkip)
			{
				OutResults.Add(Objects[I].UserData);
			}
		}
	}

	inline void BuildExpectedResults(const TArray<FObjectData>& Objects, const ESpatialCategory CategoryToInclude, TArray<FUserDataType>& OutResults)
	{
		for (const FObjectData& Object : Objects)
		{
			if (Object.Classification.GetCategory() == CategoryToInclude)
			{
				OutResults.Add(Object.UserData);
			}
		}
	}

	inline void AddUserData(const TArray<FObjectData>& Objects, const ESpatialCategoryMask CategoriesToInclude, const int32 IndexToSkip, TArray<FUserDataType>& OutResults)
	{
		for (int32 I = 0; I < Objects.Num(); ++I)
		{
			if (I != IndexToSkip && EnumHasAnyFlags(CategoriesToInclude, ToCategoryMask(Objects[I].Classification.GetCategory())))
			{
				OutResults.Add(Objects[I].UserData);
			}
		}
	}

	inline void AddUserData(const TArray<FObjectData>& Objects, const ESpatialCategoryMask CategoriesToInclude, TArray<FUserDataType>& OutResults)
	{
		AddUserData(Objects, CategoriesToInclude, INDEX_NONE, OutResults);
	}

	inline void BuildExpectedResults(const TArray<FObjectData>& Objects, const ESpatialCategoryMask CategoriesToInclude, const int32 IndexToSkip, TArray<FUserDataType>& OutResults)
	{
		AddUserData(Objects, CategoriesToInclude, IndexToSkip, OutResults);
	}

	inline void BuildExpectedResults(const TArray<FObjectData>& Objects, const ESpatialCategoryMask CategoriesToInclude, TArray<FUserDataType>& OutResults)
	{
		BuildExpectedResults(Objects, CategoriesToInclude, INDEX_NONE, OutResults);
	}

	inline TArray<FSpatialClassification> GetClassifications()
	{
		const TArray<FSpatialClassification> Result
		{
			FSpatialClassification(ESpatialCategory::Static),
			FSpatialClassification(ESpatialCategory::Kinematic),
			FSpatialClassification(ESpatialCategory::Dynamic),
		};
		return Result;
	}

	inline const TArray<ESpatialCategoryMask> GetTestMasks()
	{
		const TArray<ESpatialCategoryMask> Result
		{
			ESpatialCategoryMask::Static,
			ESpatialCategoryMask::Kinematic,
			ESpatialCategoryMask::Dynamic,
			ESpatialCategoryMask::All
		};
		return Result;
	}

	inline int32 GenerateCategoryIndex(const TArray<FSpatialClassification>& Classifications)
	{
		const int ClassificationCount = Classifications.Num();
		REQUIRE(ClassificationCount != 0);
		const int32 CategoryIndex = GENERATE_COPY(Catch::Generators::range(0, ClassificationCount));
		return CategoryIndex;
	}

	inline int32 GenerateCategoryOffset(const TArray<FSpatialClassification>& Classifications)
	{
		const int ClassificationCount = Classifications.Num();
		REQUIRE(ClassificationCount != 0);
		return GENERATE_COPY(Catch::Generators::range(1, ClassificationCount));
	}

	inline std::ostream& operator<<(std::ostream& Stream, const ESpatialCategory Category)
	{
		Stream << ToString(Category);
		return Stream;
	}

	// This provides `tostring` functionality for catch2 when the test fails.
	inline std::ostream& operator<<(std::ostream& Stream, const ESpatialCategoryMask Mask)
	{
		Stream << ToString(Mask);
		return Stream;
	}

	template <typename SpatialPartitionType, typename SpatialPartitionCreationFunction>
	struct TPerfContext
	{
		TArray<FObjectData> Objects;
		TUniquePtr<SpatialPartitionType> SpatialPartition;

		void Setup(SpatialPartitionCreationFunction CreationFn, const TArray<FSpatialClassification>& Classifications, int32 ObjectCount)
		{
			BuildObjectList(Objects, ObjectCount, Classifications);
			SpatialPartition.Reset(CreationFn());
		}

		static void Setup(TArray<TPerfContext>& OutContexts, SpatialPartitionCreationFunction CreationFn, const int32 ContextsCount, const TArray<FSpatialClassification>& Classifications, const int32 ObjectCount)
		{
			OutContexts.SetNum(ContextsCount);
			for (int32 I = 0; I < ContextsCount; ++I)
			{
				OutContexts[I].Setup(CreationFn, Classifications, ObjectCount);
			}
		}

		static void BuildAllSpatialPartitions(TArray<TPerfContext>& Contexts)
		{
			for (TPerfContext& Context : Contexts)
			{
				BuildFromObjects(*Context.SpatialPartition, Context.Objects);
			}
		}
	};

	template <typename SpatialPartitionType>
	void TestCollectionBasicBuild(SpatialPartitionType& SpatialPartition, const TArray<FSpatialClassification> Classifications = GetClassifications(), const TArray<ESpatialCategoryMask> MasksToTest = GetTestMasks())
	{
		// Setup with one object in each category
		TArray<FObjectData> Objects;
		BuildObjectList(Objects, Classifications.Num(), Classifications);
		BuildFromObjects(SpatialPartition, Objects);

		for (ESpatialCategoryMask Mask : MasksToTest)
		{
			CAPTURE(Mask);
			TArray<FUserDataType> ExpectedResults;
			BuildExpectedResults(Objects, Mask, ExpectedResults);

			{
				INFO("Overlap");
				FOverlapQueryData QueryData(FAABB3(FVec3(0), FVec3(3)));
				FTestOverlapCollectorVisitor Visitor;
				SpatialPartition.Overlap(QueryData, Visitor, Mask);
				CHECK(Visitor.Results == ExpectedResults);
			}
			{
				INFO("Raycast");
				FRaycastQueryData QueryData
				{
					.Start = FVec3(-1, 0.5f, 0.5f),
					.Direction = FVec3(1, 0, 0),
					.Length = 20,
				};
				FTestRaycastCollectorVisitor Visitor;
				SpatialPartition.Raycast(QueryData, Visitor, Mask);
				CHECK(Visitor.Results == ExpectedResults);
			}
			{
				INFO("Sweep");
				FSweepQueryData QueryData
				{
					.Start = FVec3(-1, 0.5f, 0.5f),
					.Direction = FVec3(1, 0, 0),
					.HalfExtents = FVec3(1),
					.Length = 20,
				};
				FTestSweepCollectorVisitor Visitor;
				SpatialPartition.Sweep(QueryData, Visitor, Mask);
				CHECK(Visitor.Results == ExpectedResults);
			}
		}
	}

	template <typename SpatialPartitionType>
	void TestCollectionBasicInsert(SpatialPartitionType& SpatialPartition, const TArray<FSpatialClassification> Classifications = GetClassifications(), const TArray<ESpatialCategoryMask> MasksToTest = GetTestMasks())
	{
		const FVec3 AabbSize(1);

		// Setup 1 object of each category inside the collection
		TArray<FObjectData> Objects;
		BuildObjectList(Objects, Classifications.Num(), Classifications, AabbSize);
		BuildFromObjects(SpatialPartition, Objects);

		// Test adding one new object for each category
		const int32 CategoryIndex = GenerateCategoryIndex(Classifications);
		CAPTURE(Classifications[CategoryIndex]);
		const int32 TestIndex = Objects.Emplace(FObjectData{ .Aabb = BuildAabbCenterExtents(FVec3(3, 0, 0), AabbSize), .UserData = Classifications.Num() + 1, .Classification = Classifications[CategoryIndex] });
		InsertObject(SpatialPartition, Objects[TestIndex]);

		for (ESpatialCategoryMask Mask : MasksToTest)
		{
			CAPTURE(Mask);
			TArray<FUserDataType> ExpectedResults;
			BuildExpectedResults(Objects, Mask, ExpectedResults);

			{
				INFO("Overlap");
				FOverlapQueryData QueryData(FAABB3(FVec3(0), FVec3(4)));
				FTestOverlapCollectorVisitor Visitor;
				SpatialPartition.Overlap(QueryData, Visitor, Mask);
				CHECK(Visitor.Results == ExpectedResults);
			}
			{
				INFO("Raycast");
				FRaycastQueryData QueryData
				{
					.Start = FVec3(-1, 0.5f, 0.5f),
					.Direction = FVec3(1, 0, 0),
					.Length = 20,
				};
				FTestRaycastCollectorVisitor Visitor;
				SpatialPartition.Raycast(QueryData, Visitor, Mask);
				CHECK(Visitor.Results == ExpectedResults);
			}
			{
				INFO("Sweep");
				FSweepQueryData QueryData
				{
					.Start = FVec3(-1, 0.5f, 0.5f),
					.Direction = FVec3(1, 0, 0),
					.HalfExtents = FVec3(1),
					.Length = 20,
				};
				FTestSweepCollectorVisitor Visitor;
				SpatialPartition.Sweep(QueryData, Visitor, Mask);
				CHECK(Visitor.Results == ExpectedResults);
			}
		}
	}

	template <typename SpatialPartitionType>
	void TestCollectionBasicUpdateSameCategory(SpatialPartitionType& SpatialPartition, const TArray<FSpatialClassification> Classifications = GetClassifications(), const TArray<ESpatialCategoryMask> MasksToTest = GetTestMasks())
	{
		const FVec3 UpdateOffset(0, 5, 0);

		// Setup 2 objects of each category inside the collection
		TArray<FObjectData> Objects;
		BuildObjectList(Objects, Classifications.Num() * 2, Classifications);
		BuildFromObjects(SpatialPartition, Objects);

		// Update one object by moving its aabb.
		const int32 IndexToUpdate = GenerateCategoryIndex(Classifications);
		CAPTURE(Objects[IndexToUpdate].Classification.GetCategory());
		Objects[IndexToUpdate].Aabb.MoveByVector(UpdateOffset);
		UpdateObject(SpatialPartition, Objects[IndexToUpdate]);

		for (ESpatialCategoryMask Mask : MasksToTest)
		{
			CAPTURE(Mask);
			TArray<FUserDataType> ExpectedResults0;
			BuildExpectedResults(Objects, Mask, IndexToUpdate, ExpectedResults0);
			TArray<FUserDataType> ExpectedResults1;
			if (EnumHasAnyFlags(Mask, ToCategoryMask(Objects[IndexToUpdate].Classification.GetCategory())))
			{
				ExpectedResults1.Add(Objects[IndexToUpdate].UserData);
			}

			{
				INFO("Overlap");
				FOverlapQueryData QueryData(FAABB3(FVec3(0), FVec3(6, 1, 1)));
				{
					INFO("Old Location");
					FTestOverlapCollectorVisitor Visitor;
					SpatialPartition.Overlap(QueryData, Visitor, Mask);
					CHECK(Visitor.Results == ExpectedResults0);
				}
				{
					INFO("New Location");
					FTestOverlapCollectorVisitor Visitor;
					QueryData.Aabb.MoveByVector(UpdateOffset);
					SpatialPartition.Overlap(QueryData, Visitor, Mask);
					CHECK(Visitor.Results == ExpectedResults1);
				}
			}
			{
				INFO("Raycast");
				FRaycastQueryData QueryData
				{
					.Start = FVec3(-1, 0.5f, 0.5f),
					.Direction = FVec3(1, 0, 0),
					.Length = 20,
				};
				
				{
					INFO("Old Location");
					FTestRaycastCollectorVisitor Visitor;
					SpatialPartition.Raycast(QueryData, Visitor, Mask);
					CHECK(Visitor.Results == ExpectedResults0);
				}
				{
					INFO("New Location");
					FTestRaycastCollectorVisitor Visitor;
					QueryData.Start += UpdateOffset;
					SpatialPartition.Raycast(QueryData, Visitor, Mask);
					CHECK(Visitor.Results == ExpectedResults1);
				}
			}
			{
				INFO("Sweep");
				FSweepQueryData QueryData
				{
					.Start = FVec3(-1, 0.5f, 0.5f),
					.Direction = FVec3(1, 0, 0),
					.HalfExtents = FVec3(1),
					.Length = 20,
				};
				{
					INFO("Old Location");
					FTestSweepCollectorVisitor Visitor;
					SpatialPartition.Sweep(QueryData, Visitor, Mask);
					CHECK(Visitor.Results == ExpectedResults0);
				}
				{
					INFO("New Location");
					FTestSweepCollectorVisitor Visitor;
					QueryData.Start += UpdateOffset;
					SpatialPartition.Sweep(QueryData, Visitor, Mask);
					CHECK(Visitor.Results == ExpectedResults1);
				}
			}
		}
	}

	template <typename SpatialPartitionType>
	void TestCollectionBasicUpdateDifferentCategory(SpatialPartitionType& SpatialPartition, const TArray<FSpatialClassification> Classifications = GetClassifications())
	{
		// This test doesn't work if there's only one classification
		check(Classifications.Num() > 1);

		const FVec3 AabbSize(1);

		// Setup one object in one category
		const int32 ClassificationIndex = GenerateCategoryIndex(Classifications);
		const int32 NewClassificationOffset = GenerateCategoryOffset(Classifications);
		const int32 NewClassificationIndex = (ClassificationIndex + NewClassificationOffset) % Classifications.Num();
		CAPTURE(Classifications[ClassificationIndex]);
		CAPTURE(Classifications[NewClassificationIndex]);
		FObjectData Object{ .Aabb = BuildAabbCenterExtents(FVec3(0), AabbSize), .UserData = 0, .Classification = Classifications[ClassificationIndex] };
		InsertObject(SpatialPartition, Object);

		// Change the classification to one of the other categories and verify the results.
		Object.Classification = Classifications[NewClassificationIndex];
		UpdateObject(SpatialPartition, Object);

		const TArray<FUserDataType> ExpectedResults{ Object.UserData };

		{
			INFO("Overlap");
			FTestOverlapCollectorVisitor Visitor;
			FOverlapQueryData QueryData(FAABB3(FVec3(0), FVec3(6, 1, 1)));
			SpatialPartition.Overlap(QueryData, Visitor, ESpatialCategoryMask::All);
			CHECK(Visitor.Results == ExpectedResults);
		}
		{
			INFO("Raycast");
			FTestRaycastCollectorVisitor Visitor;
			FRaycastQueryData QueryData
			{
				.Start = FVec3(-1, 0.5f, 0.5f),
				.Direction = FVec3(1, 0, 0),
				.Length = 20,
			};
			SpatialPartition.Raycast(QueryData, Visitor, ESpatialCategoryMask::All);
			CHECK(Visitor.Results == ExpectedResults);
		}
		{
			INFO("Sweep All");
			FTestSweepCollectorVisitor Visitor;
			FSweepQueryData QueryData
			{
				.Start = FVec3(-1, 0.5f, 0.5f),
				.Direction = FVec3(1, 0, 0),
				.HalfExtents = FVec3(1),
				.Length = 20,
			};
			SpatialPartition.Sweep(QueryData, Visitor, ESpatialCategoryMask::All);
			CHECK(Visitor.Results == ExpectedResults);
		}
	}

	template <typename SpatialPartitionType>
	void TestCollectionBasicRemove(SpatialPartitionType& SpatialPartition, const TArray<FSpatialClassification> Classifications = GetClassifications(), const TArray<ESpatialCategoryMask> MasksToTest = GetTestMasks())
	{
		// Setup two objects of each category inside the collection.
		TArray<FObjectData> Objects;
		BuildObjectList(Objects, Classifications.Num() * 2, Classifications);
		BuildFromObjects(SpatialPartition, Objects);

		// Run a test where one object is removed (try each category).
		const int32 IndexToRemove = GenerateCategoryIndex(Classifications);
		CAPTURE(Objects[IndexToRemove].Classification.GetCategory());
		RemoveObject(SpatialPartition, Objects[IndexToRemove]);

		for (ESpatialCategoryMask Mask : MasksToTest)
		{
			CAPTURE(Mask);
			TArray<FUserDataType> ExpectedResults;
			BuildExpectedResults(Objects, Mask, IndexToRemove, ExpectedResults);

			{
				INFO("Overlap");
				FOverlapQueryData QueryData(FAABB3(FVec3(0), FVec3(6)));
				FTestOverlapCollectorVisitor Visitor;
				SpatialPartition.Overlap(QueryData, Visitor, Mask);
				CHECK(Visitor.Results == ExpectedResults);
			}
			{
				INFO("Raycast");
				FRaycastQueryData QueryData
				{
					.Start = FVec3(-1, 0.5f, 0.5f),
					.Direction = FVec3(1, 0, 0),
					.Length = 20,
				};
				FTestRaycastCollectorVisitor Visitor;
				SpatialPartition.Raycast(QueryData, Visitor, Mask);
				CHECK(Visitor.Results == ExpectedResults);
			}
			{
				INFO("Sweep");
				FSweepQueryData QueryData
				{
					.Start = FVec3(-1, 0.5f, 0.5f),
					.Direction = FVec3(1, 0, 0),
					.HalfExtents = FVec3(1),
					.Length = 20,
				};
				FTestSweepCollectorVisitor Visitor;
				SpatialPartition.Sweep(QueryData, Visitor, Mask);
				CHECK(Visitor.Results == ExpectedResults);
			}
		}
	}

	template <typename SpatialPartitionType>
	void TestCollectionBasicSelfQuery(SpatialPartitionType& SpatialPartition, const TArray<FSpatialClassification> Classifications = GetClassifications())
	{
		// TODO: Need to figure out how to test the "all" category mask. 
		// This is typically only really used for dynamic though so for now this test is reasonable.

		using FPair = FTestSelfQueryCollectorVisitor::FPair;
		// Setup two objects of each category inside the collection.
		TArray<FObjectData> Objects;
		BuildObjectList(Objects, Classifications.Num(), Classifications);
		BuildFromObjects(SpatialPartition, Objects);

		// Should be no results to start
		for (const FSpatialClassification& Classification : Classifications)
		{
			FTestSelfQueryCollectorVisitor Visitor;
			SpatialPartition.SelfQuery(Visitor, ToCategoryMask(Classification.GetCategory()));
			CHECK(Visitor.Results.IsEmpty());
		}

		const int32 ObjectCategoryIndex = GenerateCategoryIndex(Classifications);
		const int32 TestIndex = Objects.Emplace(FObjectData{ .Aabb = FAABB3(FVec3(0, 0, 0), FVec3(3, 1, 1)), .UserData = 3, .Classification = Classifications[ObjectCategoryIndex] });
		FObjectData& TestObject = Objects[TestIndex];
		InsertObject(SpatialPartition, TestObject);

		// Now we expect results only in the category we added
		for (int32 I = 0; I < Classifications.Num(); ++I)
		{
			FTestSelfQueryCollectorVisitor Visitor;
			SpatialPartition.SelfQuery(Visitor, ToCategoryMask(Classifications[I].GetCategory()));
			if (I == ObjectCategoryIndex)
			{
				const TArray<FPair> Expected{ FPair {Objects[ObjectCategoryIndex].UserData, TestObject.UserData} };
				CHECK(Visitor.Results == Expected);
			}
			else
			{
				CHECK(Visitor.Results.IsEmpty());
			}
		}

		// Now update the object so it isn't overlapping anything
		TestObject.Aabb.MoveByVector(FVec3(0, 5, 0));
		UpdateObject(SpatialPartition, TestObject);
		for (const FSpatialClassification& Classification : Classifications)
		{
			FTestSelfQueryCollectorVisitor Visitor;
			SpatialPartition.SelfQuery(Visitor, ToCategoryMask(Classification.GetCategory()));
			CHECK(Visitor.Results.IsEmpty());
		}

		// Move back, we should get results again
		TestObject.Aabb.MoveByVector(FVec3(0, -5, 0));
		UpdateObject(SpatialPartition, TestObject);
		for (int32 I = 0; I < Classifications.Num(); ++I)
		{
			FTestSelfQueryCollectorVisitor Visitor;
			SpatialPartition.SelfQuery(Visitor, ToCategoryMask(Classifications[I].GetCategory()));
			if (I == ObjectCategoryIndex)
			{
				const TArray<FPair> Expected{ FPair {Objects[ObjectCategoryIndex].UserData, TestObject.UserData} };
				CHECK(Visitor.Results == Expected);
			}
			else
			{
				CHECK(Visitor.Results.IsEmpty());
			}
		}

		// Remove the object so we should have no results
		RemoveObject(SpatialPartition, TestObject);
		for (const FSpatialClassification& Classification : Classifications)
		{
			FTestSelfQueryCollectorVisitor Visitor;
			SpatialPartition.SelfQuery(Visitor, ToCategoryMask(Classification.GetCategory()));
			CHECK(Visitor.Results.IsEmpty());
		}
	}

	template <typename SpatialPartitionType, typename SpatialPartitionCreationFunction>
	void TestCollectionBasicInsertPerf(SpatialPartitionCreationFunction CreationFn, const TArray<FSpatialClassification> Classifications = GetClassifications(), const int32 ObjectsPerClassification = 1000)
	{
		using FPerfContext = TPerfContext<SpatialPartitionType, SpatialPartitionCreationFunction>;
		const int32 ObjectCount = Classifications.Num() * ObjectsPerClassification;

		BENCHMARK_ADVANCED("Insert")(Catch::Benchmark::Chronometer meter)
		{
			TArray<FPerfContext> Contexts;
			FPerfContext::Setup(Contexts, CreationFn, meter.runs(), Classifications, ObjectCount);

			meter.measure([&Contexts](int I)
			{
				FPerfContext& Context = Contexts[I];
				for (FObjectData& Object : Context.Objects)
				{
					InsertObject(*Context.SpatialPartition, Object);
				}
			});
		};
	}

	template <typename SpatialPartitionType, typename SpatialPartitionCreationFunction>
	void TestCollectionBasicUpdatePerf(SpatialPartitionCreationFunction CreationFn, const TArray<FSpatialClassification> Classifications = GetClassifications(), const int32 ObjectsPerClassification = 1000)
	{
		using FPerfContext = TPerfContext<SpatialPartitionType, SpatialPartitionCreationFunction>;
		const int32 ObjectCount = Classifications.Num() * ObjectsPerClassification;

		BENCHMARK_ADVANCED("Update")(Catch::Benchmark::Chronometer meter)
		{
			TArray<FPerfContext> Contexts;
			FPerfContext::Setup(Contexts, CreationFn, meter.runs(), Classifications, ObjectCount);
			FPerfContext::BuildAllSpatialPartitions(Contexts);
			for (FPerfContext& Context : Contexts)
			{
				for (FObjectData& Obj : Context.Objects)
				{
					Obj.Aabb.MoveByVector(FVec3(0, 10, 0));
				}
			}

			meter.measure([&Contexts](int I)
			{
				FPerfContext& Context = Contexts[I];
				for (FObjectData& Object : Context.Objects)
				{
					UpdateObject(*Context.SpatialPartition, Object);
				}
			});
		};
	}

	template <typename SpatialPartitionType, typename SpatialPartitionCreationFunction>
	void TestCollectionBasicRemovePerf(SpatialPartitionCreationFunction CreationFn, const TArray<FSpatialClassification> Classifications = GetClassifications(), const int32 ObjectsPerClassification = 1000)
	{
		using FPerfContext = TPerfContext<SpatialPartitionType, SpatialPartitionCreationFunction>;
		const int32 ObjectCount = Classifications.Num() * ObjectsPerClassification;

		BENCHMARK_ADVANCED("Remove - Dirty")(Catch::Benchmark::Chronometer meter)
		{
			TArray<FPerfContext> Contexts;
			FPerfContext::Setup(Contexts, CreationFn, meter.runs(), Classifications, ObjectCount);
			FPerfContext::BuildAllSpatialPartitions(Contexts);

			meter.measure([&Contexts](int I)
			{
				FPerfContext& Context = Contexts[I];
				for (FObjectData& Object : Context.Objects)
				{
					RemoveObject(*Context.SpatialPartition, Object);
				}
			});
		};
		BENCHMARK_ADVANCED("Remove - Pristine")(Catch::Benchmark::Chronometer meter)
		{
			TArray<FPerfContext> Contexts;
			FPerfContext::Setup(Contexts, CreationFn, meter.runs(), Classifications, ObjectCount);
			FPerfContext::BuildAllSpatialPartitions(Contexts);

			// Force all spatial partitions to be in a pristine state
			for (FPerfContext& Context : Contexts)
			{
				Rebuild(*Context.SpatialPartition);
			}

			meter.measure([&Contexts](int I)
			{
				FPerfContext& Context = Contexts[I];
				for (FObjectData& Object : Context.Objects)
				{
					RemoveObject(*Context.SpatialPartition, Object);
				}
			});
		};
	}

	template <typename SpatialPartitionType, typename SpatialPartitionCreationFunction>
	void TestCollectionBasicRemovePerfWithSplitClassifications(SpatialPartitionCreationFunction CreationFn, const int32 ObjectsPerClassification = 1000)
	{
		SECTION("Dynamic Only")
		{
			const TArray<FSpatialClassification> Classifications
			{
				FSpatialClassification(ESpatialCategory::Dynamic),
			};
			TestCollectionBasicRemovePerf<SpatialPartitionType>(CreationFn, Classifications, ObjectsPerClassification);
		}
		SECTION("Static Only")
		{

			const TArray<FSpatialClassification> Classifications
			{
				FSpatialClassification(ESpatialCategory::Static),
			};
			TestCollectionBasicRemovePerf<SpatialPartitionType>(CreationFn, Classifications, ObjectsPerClassification);
		}
		SECTION("Mixed")
		{
			// Only test dynamic + static since kinematic is grouped with static now. This would give a false bias since it'd effectively double dynamic's size.
			const TArray<FSpatialClassification> Classifications
			{
				FSpatialClassification(ESpatialCategory::Dynamic),
				FSpatialClassification(ESpatialCategory::Static),
			};
			TestCollectionBasicRemovePerf<SpatialPartitionType>(CreationFn, Classifications, ObjectsPerClassification);
		}
	}

	template <typename SpatialPartitionType, typename SpatialPartitionCreationFunction>
	void TestCollectionBasicRebuildPerf(SpatialPartitionCreationFunction CreationFn, const TArray<FSpatialClassification> Classifications = GetClassifications(), const int32 ObjectsPerClassification = 1000)
	{
		using FPerfContext = TPerfContext<SpatialPartitionType, SpatialPartitionCreationFunction>;
		const int32 ObjectCount = Classifications.Num() * ObjectsPerClassification;

		BENCHMARK_ADVANCED("Rebuild")(Catch::Benchmark::Chronometer meter)
		{
			TArray<FPerfContext> Contexts;
			FPerfContext::Setup(Contexts, CreationFn, meter.runs(), Classifications, ObjectCount);
			FPerfContext::BuildAllSpatialPartitions(Contexts);

			meter.measure([&Contexts](int I)
			{
				Rebuild(*Contexts[I].SpatialPartition);
			});
		};
	}

	template <typename SpatialPartitionType>
	void TestCollectionBasicOverlapPerf(SpatialPartitionType& SpatialPartition, const TArray<FSpatialClassification> Classifications = GetClassifications(), const int32 ObjectsPerClassification = 1000)
	{
		const int32 ObjectCount = Classifications.Num() * ObjectsPerClassification;
		TArray<FObjectData> Objects;
		BuildObjectList(Objects, ObjectCount, Classifications);
		BuildFromObjects(SpatialPartition, Objects);

		// Move everything, causing the spatial partition to be in a dirty state
		for (FObjectData& Obj : Objects)
		{
			Obj.Aabb.MoveByVector(FVec3(0, 100, 0));
			UpdateObject(SpatialPartition, Obj);
		}
		BENCHMARK("Overlaps - Dirty")
		{
			for (const FObjectData& Obj : Objects)
			{
				const FOverlapQueryData QueryData(Obj.Aabb);
				FTestOverlapAccumulatorVisitor Visitor;
				SpatialPartition.Overlap(QueryData, Visitor);
			}
		};

		// Now do a clean rebuild such that we're in the best state possible
		Rebuild(SpatialPartition);
		BENCHMARK("Overlaps - Pristine")
		{
			for (const FObjectData& Obj : Objects)
			{
				const FOverlapQueryData QueryData(Obj.Aabb);
				FTestOverlapAccumulatorVisitor Visitor;
				SpatialPartition.Overlap(QueryData, Visitor);
			}
		};
	}
} // namespace Chaos::SpatialPartition
