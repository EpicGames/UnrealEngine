// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ChaosSpatialPartitions/Common.h"

namespace Chaos::SpatialPartition
{
	inline FAABB3 BuildAabbMinExtents(const FVec3& Min, const FVec3& Extents)
	{
		return FAABB3(Min, Min + Extents);
	}

	inline FAABB3 BuildAabbCenterExtents(const FVec3& Center, const FVec3& Extents)
	{
		const FVec3 Min = Center - Extents * 0.5f;
		const FVec3 Max = Center + Extents * 0.5f;
		return FAABB3(Min, Max);
	}

	// Build the index for a 1D array using 3D coordinates.
	inline int32 GetIndex(const int32 X, const int32 Y, const int32 Z, const int32 CountX, const int32 CountY, const int32 /*CountZ*/)
	{
		const int32 Index = X + Y * CountX + Z * (CountX * CountY);
		return Index;
	}
} // namespace Chaos::SpatialPartition
