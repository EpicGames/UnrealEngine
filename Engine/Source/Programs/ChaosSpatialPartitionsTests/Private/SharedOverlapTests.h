// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "ChaosTestHarness.h"

#include "TestVisitors.h"

#include "ChaosSpatialPartitions/SpatialHandle.h"
#include "ObjectData.h"
#include "SharedHelpers.h"

namespace Chaos::SpatialPartition
{
	template <typename SpatialPartitionType>
	void TestOverlapBuildBasic(SpatialPartitionType& SpatialPartition)
	{
		// For basic build tests, we want to test:
		// - An empty spatial partition
		// - An empty result list
		// - Single object (each object)
		// - Multiple results

		const FVec3 Spacing(1);
		const FVec3 AabbSize(1);
		TArray<FObjectData> Objects;
		BuildObjectList(Objects, 2, 2, 2, Spacing, AabbSize);

		FTestOverlapCollectorVisitor Visitor;
		SECTION("Overlap: Empty Spatial Partition")
		{
			FOverlapQueryRuntimeData QueryRuntimeData(FAABB3::ZeroAABB());

			CHECK(EVisitResult::Continue == SpatialPartition.Overlap(QueryRuntimeData, Visitor));
			CHECK(Visitor.Results.IsEmpty());
		}

		BuildFromObjects(SpatialPartition, Objects);
		SECTION("Overlap: Empty Results")
		{
			const FAABB3 TestAabb(FVec3(-5), FVec3(-4));
			FOverlapQueryRuntimeData QueryRuntimeData(TestAabb);

			CHECK(EVisitResult::Continue == SpatialPartition.Overlap(QueryRuntimeData, Visitor));
			CHECK(Visitor.Results.IsEmpty());
		}
		for (int32 Index = 0; Index < Objects.Num(); ++Index)
		{
			DYNAMIC_SECTION("Overlap: Hit Object " << Index)
			{
				const FObjectData& TestObject = Objects[Index];
				FOverlapQueryRuntimeData QueryRuntimeData(TestObject.Aabb);

				CHECK(EVisitResult::Continue == SpatialPartition.Overlap(QueryRuntimeData, Visitor));
				CHECK(Visitor.Results == TArray<FUserDataType>{TestObject.UserData});
			}
		}
		SECTION("Overlap: Multiple Results")
		{
			const FAABB3 TestAabb(Objects[0].Aabb.Min(), Objects.Last().Aabb.Max());
			FOverlapQueryRuntimeData QueryRuntimeData(TestAabb);
			TArray<FUserDataType> Expected;
			for (const FObjectData& Obj : Objects)
			{
				Expected.Add(Obj.UserData);
			}

			CHECK(EVisitResult::Continue == SpatialPartition.Overlap(QueryRuntimeData, Visitor));
			CHECK(Visitor.Results == Expected);
		}
	}

	template <typename SpatialPartitionType>
	void TestOverlapInsert(SpatialPartitionType& SpatialPartition)
	{
		constexpr int32 Count = 4;
		const FVec3 Spacing(1);
		TArray<FObjectData> Objects;
		BuildObjectList(Objects, Count, 1, 1, Spacing);

		// Build a test aabb that overlaps 1 and 2. This will mix queries where some objects are in the results and some aren't as we incrementally insert.
		const FAABB3 TestAabb(Objects[1].Aabb.Min(), Objects[2].Aabb.Max());
		FOverlapQueryRuntimeData QueryRuntimeData(TestAabb);
		FTestOverlapCollectorVisitor Visitor;

		InsertObject(SpatialPartition, Objects[0]);
		Visitor.Reset();
		CHECK(EVisitResult::Continue == SpatialPartition.Overlap(QueryRuntimeData, Visitor));
		CHECK(Visitor.Results.IsEmpty());

		InsertObject(SpatialPartition, Objects[1]);
		Visitor.Reset();
		CHECK(EVisitResult::Continue == SpatialPartition.Overlap(QueryRuntimeData, Visitor));
		CHECK(Visitor.Results == TArray<FUserDataType>{Objects[1].UserData});

		InsertObject(SpatialPartition, Objects[2]);
		Visitor.Reset();
		CHECK(EVisitResult::Continue == SpatialPartition.Overlap(QueryRuntimeData, Visitor));
		CHECK(Visitor.Results == TArray<FUserDataType>{Objects[1].UserData, Objects[2].UserData});

		InsertObject(SpatialPartition, Objects[3]);
		Visitor.Reset();
		CHECK(EVisitResult::Continue == SpatialPartition.Overlap(QueryRuntimeData, Visitor));
		CHECK(Visitor.Results == TArray<FUserDataType>{Objects[1].UserData, Objects[2].UserData});
	}

	template <typename SpatialPartitionType>
	void TestOverlapRemove(SpatialPartitionType& SpatialPartition)
	{
		constexpr int32 Count = 4;
		const FVec3 Spacing(1);
		TArray<FObjectData> Objects;
		BuildObjectList(Objects, Count, 1, 1, Spacing);
		BuildFromObjects(SpatialPartition, Objects);

		// Build a test aabb that overlaps 1 and 2. This will mix queries where some objects are in the results and some aren't as we incrementally remove.
		const FAABB3 TestAabb(Objects[1].Aabb.Min(), Objects[2].Aabb.Max());
		FOverlapQueryRuntimeData QueryRuntimeData(TestAabb);
		FTestOverlapCollectorVisitor Visitor;

		RemoveObject(SpatialPartition, Objects[3]);
		Visitor.Reset();
		CHECK(EVisitResult::Continue == SpatialPartition.Overlap(QueryRuntimeData, Visitor));
		CHECK(Visitor.Results == TArray<FUserDataType>{Objects[1].UserData, Objects[2].UserData});

		RemoveObject(SpatialPartition, Objects[2]);
		Visitor.Reset();
		CHECK(EVisitResult::Continue == SpatialPartition.Overlap(QueryRuntimeData, Visitor));
		CHECK(Visitor.Results == TArray<FUserDataType>{Objects[1].UserData});

		RemoveObject(SpatialPartition, Objects[1]);
		Visitor.Reset();
		CHECK(EVisitResult::Continue == SpatialPartition.Overlap(QueryRuntimeData, Visitor));
		CHECK(Visitor.Results.IsEmpty());

		RemoveObject(SpatialPartition, Objects[0]);
		Visitor.Reset();
		CHECK(EVisitResult::Continue == SpatialPartition.Overlap(QueryRuntimeData, Visitor));
		CHECK(Visitor.Results.IsEmpty());
	}

	template <typename SpatialPartitionType>
	void TestOverlapUpdateBasic(SpatialPartitionType& SpatialPartition)
	{
		// This test starts with all objects in a region and moves half of them to a new region. 
		// This tests both the before and after locations, and does so before and after the moves.
		constexpr int32 Count = 4;
		const FVec3 Spacing(1);
		TArray<FObjectData> Objects;
		BuildObjectList(Objects, Count, 1, 1, Spacing);
		BuildFromObjects(SpatialPartition, Objects);

		const FVec3 Offset(0, 5, 0);
		const FAABB3 OldLocationTestAabb(Objects[0].Aabb.Min(), Objects.Last().Aabb.Max());
		const FAABB3 NewLocationTestAabb(OldLocationTestAabb.Min() + Offset, OldLocationTestAabb.Max() + Offset);
		FOverlapQueryRuntimeData NewQueryData(NewLocationTestAabb);
		FOverlapQueryRuntimeData OldQueryData(OldLocationTestAabb);
		FTestOverlapCollectorVisitor Visitor;

		// First verify the original query results. We expect all of the objects in the "old" location and none in the "new"
		Visitor.Reset();
		CHECK(EVisitResult::Continue == SpatialPartition.Overlap(OldQueryData, Visitor));
		CHECK(Visitor.Results == TArray<FUserDataType>{Objects[0].UserData, Objects[1].UserData, Objects[2].UserData, Objects[3].UserData});

		Visitor.Reset();
		CHECK(EVisitResult::Continue == SpatialPartition.Overlap(NewQueryData, Visitor));
		CHECK(Visitor.Results.IsEmpty());

		// Now move some of the objects and re-test.
		Objects[0].Aabb.MoveByVector(Offset);
		Objects[2].Aabb.MoveByVector(Offset);
		UpdateObject(SpatialPartition, Objects[0]);
		UpdateObject(SpatialPartition, Objects[2]);

		Visitor.Reset();
		CHECK(EVisitResult::Continue == SpatialPartition.Overlap(NewQueryData, Visitor));
		CHECK(Visitor.Results == TArray<FUserDataType>{Objects[0].UserData, Objects[2].UserData});

		Visitor.Reset();
		CHECK(EVisitResult::Continue == SpatialPartition.Overlap(OldQueryData, Visitor));
		CHECK(Visitor.Results == TArray<FUserDataType>{Objects[1].UserData, Objects[3].UserData});
	}

	template <typename SpatialPartitionType>
	void TestOverlapUpdateWithinBounds(SpatialPartitionType& SpatialPartition)
	{
		constexpr int32 Count = 4;
		const FVec3 Spacing(1);

		TArray<FObjectData> Objects;
		BuildObjectList(Objects, Count, 1, 1, Spacing);
		BuildFromObjects(SpatialPartition, Objects);

		FObjectData& TestObject = Objects.Last();
		const FVec3 AabbSize = TestObject.Aabb.Extents();
		const FVec3 AabbMin = TestObject.Aabb.Min();
		const FVec3 AabbMax = TestObject.Aabb.Max();
		const FVec3 SmallSize = AabbSize / 8;

		// Update the aabb to be 1/8 the size and position at the bottom "min" corner.
		TestObject.Aabb = FAABB3(AabbMin, AabbMin + SmallSize);
		UpdateObject(SpatialPartition, TestObject);

		FTestOverlapCollectorVisitor Visitor;
		// Test the new aabb to make sure we get the object.
		FOverlapQueryRuntimeData NewQueryData(TestObject.Aabb);
		CHECK(EVisitResult::Continue == SpatialPartition.Overlap(NewQueryData, Visitor));
		CHECK(Visitor.Results == TArray<FUserDataType>{TestObject.UserData});

		// Test something within the old aabb but not hitting the new aabb.
		Visitor.Results.Reset();
		const FAABB3 TestAabb(AabbMax - SmallSize, AabbMax);
		FOverlapQueryRuntimeData OldQueryData(TestAabb);
		CHECK(EVisitResult::Continue == SpatialPartition.Overlap(OldQueryData, Visitor));
		CHECK(Visitor.Results.IsEmpty());
	}

	template <typename SpatialPartitionType>
	void TestOverlapWithEarlyVisitResultTermination(SpatialPartitionType& SpatialPartition)
	{
		constexpr int32 Count = 4;

		TArray<FObjectData> Objects;
		BuildObjectList(Objects, Count, 1, 1);
		BuildFromObjects(SpatialPartition, Objects);

		// Build a query that hits every object, however specify to terminate the query after 2 visits.
		FTestOverlapCollectorVisitor Visitor;
		Visitor.MaxResults = 2;
		FOverlapQueryRuntimeData QueryRuntimeData(FAABB3(Objects[0].Aabb.Min(), Objects.Last().Aabb.Max()));
		CHECK(EVisitResult::Stop == SpatialPartition.Overlap(QueryRuntimeData, Visitor));
		// Note: We can't be guaranteed what results we actually get, instead just verify the number of results.
		CHECK(Visitor.Results.Num() == 2);
	}

	template <typename SpatialPartitionType>
	void TestOverlapPerformanceEmpty(SpatialPartitionType& SpatialPartition)
	{
		FOverlapQueryRuntimeData QueryData(FAABB3::ZeroAABB());
		FTestOverlapAccumulatorVisitor Visitor;
		BENCHMARK("Overlap: Empty")
		{
			SpatialPartition.Overlap(QueryData, Visitor);
			return Visitor.Result;
		};
	}

	template <typename SpatialPartitionType>
	void TestOverlapPerformanceBasic(SpatialPartitionType& SpatialPartition, const int32 CountX = 100, const int32 CountY = 100, const int32 CountZ = 100)
	{
		ensure(CountX * CountY * CountZ > 1);
		const FVec3 Spacing(1);

		TArray<FObjectData> Objects;
		BuildObjectList(Objects, CountX, CountY, CountZ, Spacing);
		const FVec3 Min = Objects[0].Aabb.Min();
		const FVec3 Max = Objects.Last().Aabb.Max();

		BuildFromObjects(SpatialPartition, Objects);

		FTestOverlapAccumulatorVisitor Visitor;

		// Test having no overlaps:
		// Test a far away query. For hierarchical spatial partitions this should terminate very early.
		{
			FOverlapQueryRuntimeData QueryData(FAABB3(FVec3(-10, -10, -10), FVec3(-9, -9, -9)));
			BENCHMARK("Overlap: No Overlaps: Far away query")
			{
				Visitor.Reset();
				SpatialPartition.Overlap(QueryData, Visitor);
				return Visitor.Result;
			};
		}
		// Test a query in-between a bunch of objects. 
		// This helps to test hierarchical spatial partitions as a good chunk of the hierarchy will have to be traversed.
		{
			const FObjectData& Object0 = Objects[CountX / 2];
			const FObjectData& Object1 = Objects[CountX / 2 + 1];
			const FVec3 MidPoint = (Object0.Aabb.GetCenter() + Object1.Aabb.GetCenter()) * 0.5f;
			const FAABB3 TestAabb(FVec3(MidPoint.X, Min.Y, Min.Z), FVec3(MidPoint.X, Max.Y, Min.Z));
			FOverlapQueryRuntimeData QueryData(TestAabb);
			BENCHMARK("Overlap: No Overlaps: Query between objects")
			{
				Visitor.Reset();
				SpatialPartition.Overlap(QueryData, Visitor);
				return Visitor.Result;
			};
		}

		// Test some queries that result in a single object.
		{
			FOverlapQueryRuntimeData QueryData(Objects[0].Aabb);
			BENCHMARK("Overlap: Single Overlap: First Object")
			{
				Visitor.Reset();
				SpatialPartition.Overlap(QueryData, Visitor);
				return Visitor.Result;
			};
		}
		{
			FOverlapQueryRuntimeData QueryData(Objects[Objects.Num() / 2].Aabb);
			BENCHMARK("Overlap: Single Overlap: Middle Object")
			{
				Visitor.Reset();
				SpatialPartition.Overlap(QueryData, Visitor);
				return Visitor.Result;
			};
		}
		{
			FOverlapQueryRuntimeData QueryData(Objects.Last().Aabb);
			BENCHMARK("Overlap: Single Overlap: Last Object")
			{
				Visitor.Reset();
				SpatialPartition.Overlap(QueryData, Visitor);
				return Visitor.Result;
			};
		}

		// Test queries that result in multiple overlaps.
		{
			// Overlap a slice in the middle. This helps to stress hierarchical spatial partitions.
			const FObjectData& Object = Objects[CountX / 2];
			const FAABB3 TestAabb(FVec3(Object.Aabb.Min().X, Min.Y, Min.Z), FVec3(Object.Aabb.Max().X, Max.Y, Min.Z));
			FOverlapQueryRuntimeData QueryData(TestAabb);
			BENCHMARK("Overlap: Multiple Overlaps: y-z plane")
			{
				Visitor.Reset();
				SpatialPartition.Overlap(QueryData, Visitor);
				return Visitor.Result;
			};
		}
		{
			// Overlap a slice in the middle. This helps to stress hierarchical spatial partitions.
			const FObjectData& Object = Objects[CountY / 2];
			const FAABB3 TestAabb(FVec3(Min.X, Object.Aabb.Min().Y, Min.Z), FVec3(Max.X, Object.Aabb.Max().Y, Min.Z));
			FOverlapQueryRuntimeData QueryData(TestAabb);
			BENCHMARK("Overlap: Multiple Overlaps: x-z plane")
			{
				Visitor.Reset();
				SpatialPartition.Overlap(QueryData, Visitor);
				return Visitor.Result;
			};
		}
		{
			// Overlap a slice in the middle. This helps to stress hierarchical spatial partitions.
			const FObjectData& Object = Objects[CountZ / 2];
			const FAABB3 TestAabb(FVec3(Min.X, Min.Y, Object.Aabb.Min().Z), FVec3(Max.X, Max.Y, Object.Aabb.Max().Z));
			FOverlapQueryRuntimeData QueryData(TestAabb);
			BENCHMARK("Overlap: Multiple Overlaps: x-y plane")
			{
				Visitor.Reset();
				SpatialPartition.Overlap(QueryData, Visitor);
				return Visitor.Result;
			};
		}

		{
			FOverlapQueryRuntimeData QueryData(FAABB3(Min, Max));
			BENCHMARK("Overlap: Overlaps Everything")
			{
				Visitor.Reset();
				SpatialPartition.Overlap(QueryData, Visitor);
				return Visitor.Result;
			};
		}
	}
} // namespace Chaos::SpatialPartition
