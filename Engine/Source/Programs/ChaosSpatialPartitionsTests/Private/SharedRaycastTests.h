// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "ChaosTestHarness.h"

#include "TestVisitors.h"

#include "ChaosSpatialPartitions/SpatialHandle.h"
#include "ObjectData.h"
#include "SharedHelpers.h"

namespace Chaos::SpatialPartition
{
	template <typename SpatialPartitionType>
	void TestRaycastBuildBasic(SpatialPartitionType& SpatialPartition)
	{
		// For basic build tests, we want to test:
		// - An empty spatial partition
		// - An empty result list
		// - Single object (each object)
		// - Multiple results

		const FVec3 AabbSize(1);
		constexpr int32 Count = 4;
		// Add a spacing of 1 between to make it so we can do easy queries that only hit 1 object.
		const FVec3 Spacing(1);
		TArray<FObjectData> Objects;
		BuildObjectList(Objects, Count, 1, 1, Spacing, AabbSize);

		FRaycastQueryData QueryData;
		QueryData.Start = FVec3::Zero();
		QueryData.Length = 10;
		QueryData.Direction = FVec3(0, -1, 0);

		FTestRaycastCollectorVisitor Visitor;
		SECTION("Raycast: Empty Spatial Partition")
		{
			FRaycastQueryRuntimeData QueryRuntimeData(QueryData);
			const EVisitResult VisitResult = SpatialPartition.Raycast(QueryRuntimeData, Visitor);

			CHECK(VisitResult == EVisitResult::Continue);
			CHECK(Visitor.Results.IsEmpty());
		}

		BuildFromObjects(SpatialPartition, Objects);
		SECTION("Raycast: Hit None")
		{
			QueryData.Start = FVec3(-1, 2, 0.5f);
			QueryData.Direction = FVec3(1, 0, 0);

			FRaycastQueryRuntimeData QueryRuntimeData(QueryData);
			const EVisitResult VisitResult = SpatialPartition.Raycast(QueryRuntimeData, Visitor);

			CHECK(VisitResult == EVisitResult::Continue);
			CHECK(Visitor.Results.IsEmpty());
		}
		for (int32 Index = 0; Index < Count; ++Index)
		{
			DYNAMIC_SECTION("Raycast: Hit Object " << Index)
			{
				const FObjectData& TestObject = Objects[Index];
				// Set the ray start half-way back along the cast direction
				QueryData.Start = TestObject.Aabb.Center() - QueryData.Direction * QueryData.Length * 0.5f;

				FRaycastQueryRuntimeData QueryRuntimeData(QueryData);
				const EVisitResult VisitResult = SpatialPartition.Raycast(QueryRuntimeData, Visitor);

				CHECK(VisitResult == EVisitResult::Continue);
				CHECK(Visitor.Results == TArray<FUserDataType>{TestObject.UserData});
			}
		}
		SECTION("Raycast: Hit All")
		{
			TArray<FUserDataType> Expected;
			for (const FObjectData& Object : Objects)
			{
				Expected.Add(Object.UserData);
			}

			QueryData.Start = FVec3(-1, 0.5f, 0.5f);
			QueryData.Direction = FVec3(1, 0, 0);

			FRaycastQueryRuntimeData QueryRuntimeData(QueryData);
			const EVisitResult VisitResult = SpatialPartition.Raycast(QueryRuntimeData, Visitor);

			CHECK(VisitResult == EVisitResult::Continue);
			CHECK(Visitor.Results == Expected);
		}
	}

	template <typename SpatialPartitionType>
	void TestRaycastInsert(SpatialPartitionType& SpatialPartition)
	{
		constexpr int32 Count = 4;
		const FVec3 Spacing(1);
		TArray<FObjectData> Objects;
		BuildObjectList(Objects, Count, 1, 1, Spacing);
		// Move Object 2 so it won't be hit by the ray.
		Objects[2].Aabb.MoveByVector(FVec3(0, 5, 0));

		FRaycastQueryData QueryData;
		// Start the raycast half-way in-between 0 and 1, this way 0 is behind the ray.
		QueryData.Start = (Objects[0].Aabb.GetCenter() + Objects[1].Aabb.GetCenter()) * 0.5f;
		QueryData.Length = 10;
		QueryData.Direction = FVec3(1, 0, 0);

		FRaycastQueryRuntimeData QueryRuntimeData(QueryData);
		FTestRaycastCollectorVisitor Visitor;

		InsertObject(SpatialPartition, Objects[0]);
		Visitor.Reset();
		CHECK(EVisitResult::Continue == SpatialPartition.Raycast(QueryRuntimeData, Visitor));
		CHECK(Visitor.Results.IsEmpty());

		InsertObject(SpatialPartition, Objects[1]);
		Visitor.Reset();
		CHECK(EVisitResult::Continue == SpatialPartition.Raycast(QueryRuntimeData, Visitor));
		CHECK(Visitor.Results == TArray<FUserDataType>{Objects[1].UserData});

		InsertObject(SpatialPartition, Objects[2]);
		Visitor.Reset();
		CHECK(EVisitResult::Continue == SpatialPartition.Raycast(QueryRuntimeData, Visitor));
		CHECK(Visitor.Results == TArray<FUserDataType>{Objects[1].UserData});

		InsertObject(SpatialPartition, Objects[3]);
		Visitor.Reset();
		CHECK(EVisitResult::Continue == SpatialPartition.Raycast(QueryRuntimeData, Visitor));
		CHECK(Visitor.Results == TArray<FUserDataType>{Objects[1].UserData, Objects[3].UserData});
	}

	template <typename SpatialPartitionType>
	void TestRaycastRemove(SpatialPartitionType& SpatialPartition)
	{
		constexpr int32 Count = 4;
		const FVec3 Spacing(1);
		TArray<FObjectData> Objects;
		BuildObjectList(Objects, Count, 1, 1, Spacing);
		// Move Object 2 so it won't be hit by the ray.
		Objects[2].Aabb.MoveByVector(FVec3(0, 5, 0));
		BuildFromObjects(SpatialPartition, Objects);

		FRaycastQueryData QueryData;
		// Start the raycast half-way in-between 0 and 1, this way 0 is behind the ray.
		QueryData.Start = (Objects[0].Aabb.GetCenter() + Objects[1].Aabb.GetCenter()) * 0.5f;
		QueryData.Length = 10;
		QueryData.Direction = FVec3(1, 0, 0);

		FRaycastQueryRuntimeData QueryRuntimeData(QueryData);
		FTestRaycastCollectorVisitor Visitor;

		RemoveObject(SpatialPartition, Objects[3]);
		Visitor.Reset();
		CHECK(EVisitResult::Continue == SpatialPartition.Raycast(QueryRuntimeData, Visitor));
		CHECK(Visitor.Results == TArray<FUserDataType>{Objects[1].UserData});

		RemoveObject(SpatialPartition, Objects[2]);
		Visitor.Reset();
		CHECK(EVisitResult::Continue == SpatialPartition.Raycast(QueryRuntimeData, Visitor));
		CHECK(Visitor.Results == TArray<FUserDataType>{Objects[1].UserData});

		RemoveObject(SpatialPartition, Objects[1]);
		Visitor.Reset();
		CHECK(EVisitResult::Continue == SpatialPartition.Raycast(QueryRuntimeData, Visitor));
		CHECK(Visitor.Results.IsEmpty());

		RemoveObject(SpatialPartition, Objects[0]);
		Visitor.Reset();
		CHECK(EVisitResult::Continue == SpatialPartition.Raycast(QueryRuntimeData, Visitor));
		CHECK(Visitor.Results.IsEmpty());
	}

	template <typename SpatialPartitionType>
	void TestRaycastUpdateBasic(SpatialPartitionType& SpatialPartition)
	{
		// This test starts with all objects in a region and moves half of them to a new region. 
		// This tests both the before and after locations, and does so before and after the moves.
		constexpr int32 Count = 4;
		const FVec3 Spacing(1);
		TArray<FObjectData> Objects;
		BuildObjectList(Objects, Count, 1, 1, Spacing);
		BuildFromObjects(SpatialPartition, Objects);

		const FVec3 Offset(0, 5, 0);
		FRaycastQueryData OldQueryData{ .Start = FVec3(-1, 0.5f, 0.5f), .Direction = FVec3(1, 0, 0), .Length = 20 };
		FRaycastQueryData NewQueryData{ .Start = OldQueryData.Start + Offset, .Direction = FVec3(1, 0, 0), .Length = 20 };
		FRaycastQueryRuntimeData OldQueryRuntimeData(OldQueryData);
		FRaycastQueryRuntimeData NewQueryRuntimeData(NewQueryData);
		FTestRaycastCollectorVisitor Visitor;

		// First verify the original query results. We expect all of the objects in the "old" location and none in the "new"
		Visitor.Reset();
		CHECK(EVisitResult::Continue == SpatialPartition.Raycast(OldQueryRuntimeData, Visitor));
		CHECK(Visitor.Results == TArray<FUserDataType>{Objects[0].UserData, Objects[1].UserData, Objects[2].UserData, Objects[3].UserData});

		Visitor.Reset();
		CHECK(EVisitResult::Continue == SpatialPartition.Raycast(NewQueryRuntimeData, Visitor));
		CHECK(Visitor.Results.IsEmpty());

		// Now move some of the objects and re-test.
		// Object 0 is moved behind the ray while object 2 is moved into the new area.
		Objects[0].Aabb.MoveByVector(FVec3(-10, 0, 0));
		Objects[2].Aabb.MoveByVector(Offset);
		UpdateObject(SpatialPartition, Objects[0]);
		UpdateObject(SpatialPartition, Objects[2]);

		Visitor.Reset();
		CHECK(EVisitResult::Continue == SpatialPartition.Raycast(NewQueryRuntimeData, Visitor));
		CHECK(Visitor.Results == TArray<FUserDataType>{Objects[2].UserData});

		Visitor.Reset();
		CHECK(EVisitResult::Continue == SpatialPartition.Raycast(OldQueryRuntimeData, Visitor));
		CHECK(Visitor.Results == TArray<FUserDataType>{Objects[1].UserData, Objects[3].UserData});
	}

	template <typename SpatialPartitionType>
	void TestRaycastUpdateWithinBounds(SpatialPartitionType& SpatialPartition)
	{
		constexpr int32 Count = 4;
		const FVec3 Spacing(1);

		TArray<FObjectData> Objects;
		BuildObjectList(Objects, Count, 1, 1, Spacing);
		BuildFromObjects(SpatialPartition, Objects);

		FObjectData& TestObject = Objects.Last();
		const FVector AabbSize = TestObject.Aabb.Extents();
		const FVec3 AabbMin = TestObject.Aabb.Min();
		const FVec3 AabbMax = TestObject.Aabb.Max();
		const FVec3 SmallSize = AabbSize / 8;

		// Update the aabb to be 1/8 the size
		TestObject.Aabb = FAABB3(AabbMin, AabbMin + SmallSize);
		UpdateObject(SpatialPartition, TestObject);

		FTestRaycastCollectorVisitor Visitor;
		FRaycastQueryData QueryData{ .Direction = FVec3(0, 0, -1), .Length = 10 };

		// Test the new bounds to make sure it's valid
		QueryData.Start = TestObject.Aabb.GetCenter() - QueryData.Direction * 5;
		FRaycastQueryRuntimeData NewQueryData(QueryData);
		CHECK(EVisitResult::Continue == SpatialPartition.Raycast(NewQueryData, Visitor));
		CHECK(Visitor.Results == TArray<FUserDataType>{TestObject.UserData});

		// Test something within the old bounds but not hitting the new bounds.
		Visitor.Results.Reset();
		QueryData.Start = AabbMax - SmallSize - QueryData.Direction * 5;
		FRaycastQueryRuntimeData OldQueryData(QueryData);
		CHECK(EVisitResult::Continue == SpatialPartition.Raycast(OldQueryData, Visitor));
		CHECK(Visitor.Results.IsEmpty());
	}

	template <typename SpatialPartitionType>
	void TestRaycastWithEarlyVisitResultTermination(SpatialPartitionType& SpatialPartition)
	{
		constexpr int32 Count = 4;

		TArray<FObjectData> Objects;
		BuildObjectList(Objects, Count);
		BuildFromObjects(SpatialPartition, Objects);

		// Build a ray that hits all of the objects, but specify to terminate after 2 visits.
		FTestRaycastCollectorVisitor Visitor;
		Visitor.MaxResults = 2;
		FRaycastQueryData QueryData{ .Start = FVec3(-5, 0.5f, 0.5f), .Direction = FVec3(1, 0, 0), .Length = 100 };
		FRaycastQueryRuntimeData QueryRuntimeData(QueryData);
		const EVisitResult VisitResult = SpatialPartition.Raycast(QueryRuntimeData, Visitor);

		CHECK(VisitResult == EVisitResult::Stop);
		CHECK(Visitor.Results.Num() == 2);
	}

	template <typename SpatialPartitionType>
	void TestRaycastWithEarlyLengthTermination(SpatialPartitionType& SpatialPartition)
	{
		constexpr int32 Count = 4;

		TArray<FObjectData> Objects;
		BuildObjectList(Objects, Count);
		BuildFromObjects(SpatialPartition, Objects);

		// Build a ray that hits all of the objects, but change the length to something really small after the first hit. 
		// This should cause all other raycasts to fail.
		FTestRaycastCollectorVisitor Visitor;
		Visitor.TargetLength = 0.1f;
		Visitor.bUpdateLength = true;
		FRaycastQueryData QueryData{ .Start = FVec3(-5, 0.5f, 0.5f), .Direction = FVec3(1, 0, 0), .Length = 100 };
		FRaycastQueryRuntimeData QueryRuntimeData(QueryData);
		const EVisitResult VisitResult = SpatialPartition.Raycast(QueryRuntimeData, Visitor);

		CHECK(VisitResult == EVisitResult::Continue);
		CHECK(Visitor.Results.Num() == 1);
	}

	template <typename SpatialPartitionType>
	void TestRaycastPerformanceEmpty(SpatialPartitionType& SpatialPartition)
	{
		FRaycastQueryData QueryData{ .Start = FVec3::Zero(), .Direction = FVec3(1, 0, 0), .Length = 10 };
		FRaycastQueryRuntimeData QueryRuntimeData(QueryData);
		FTestRaycastAccumulatorVisitor Visitor;
		BENCHMARK("Raycast: Empty")
		{
			SpatialPartition.Raycast(QueryRuntimeData, Visitor);
			return Visitor.Result;
		};
	}

	template <typename SpatialPartitionType>
	void TestRaycastPerformanceBasic(SpatialPartitionType& SpatialPartition, const int32 CountX = 10000, const int32 CountY = 10)
	{
		constexpr int32 CountZ = 1;
		const FVec3 Spacing(1);
		const FVec3 AabbSize(1, 1, 1);

		TArray<FObjectData> Objects;
		BuildObjectList(Objects, CountX, CountY, CountZ, Spacing);
		const FVec3 Min = Objects[0].Aabb.Min();
		const FVec3 Max = Objects.Last().Aabb.Max();
		const FReal RayLength = (Max - Min).Length() * 2;

		BuildFromObjects(SpatialPartition, Objects);

		FTestRaycastAccumulatorVisitor Visitor;

		// Test having no hits
		{
			// Test a ray outside the root and pointing away from everything. 
			// For hierarchical spatial partitions this should terminate very early.
			{
				FRaycastQueryData QueryData{ .Start = FVec3(-10, -10, -10), .Direction = FVec3(0, 0, -1), .Length = RayLength };
				FRaycastQueryRuntimeData QueryRuntimeData(QueryData);

				BENCHMARK("No Hits: Far away query")
				{
					Visitor.Reset();
					SpatialPartition.Raycast(QueryRuntimeData, Visitor);
					return Visitor.Result;
				};
			}
			// Test a query in-between a bunch of objects. 
			// This helps to test hierarchical spatial partitions as a good chunk of the hierarchy will have to be traversed.
			{
				// Grab an object in the middle of the y-z plane and offset on the y-axis half-way in between it and the next object.
				// Note: This assumes CountY > 1, otherwise it'll miss the root.
				const FObjectData& Object = Objects[GetIndex(0, CountY / 2, CountZ / 2, CountX, CountY, CountZ)];
				FVec3 Start = Object.Aabb.Center();
				Start.Y += (AabbSize.Y + Spacing.Y) / 2.0f;

				FRaycastQueryData QueryData{ .Start = FVec3(-10, Start.Y, Start.Z), .Direction = FVec3(1, 0, 0), .Length = RayLength };
				FRaycastQueryRuntimeData QueryRuntimeData(QueryData);
				BENCHMARK("No Hits: Query between objects")
				{
					Visitor.Reset();
					SpatialPartition.Raycast(QueryRuntimeData, Visitor);
					return Visitor.Result;
				};
			}
		}
		// Test a line along the x axis.
		{
			// Grab an object in the middle of the y-z plane. Use that to make a x-axis cast through the middle of the objects.
			const FObjectData& Object = Objects[GetIndex(0, CountY / 2, CountZ / 2, CountX, CountY, CountZ)];
			const FVec3 Center = Object.Aabb.Center();

			FRaycastQueryData QueryData{ .Start = FVec3(-10, Center.Y, Center.Z), .Direction = FVec3(1, 0, 0), .Length = RayLength };
			FRaycastQueryRuntimeData QueryRuntimeData(QueryData);
			BENCHMARK("Some Hits: Test x axis")
			{
				Visitor.Reset();
				SpatialPartition.Raycast(QueryRuntimeData, Visitor);
				return Visitor.Result;
			};
		}
		// Test a line along the y axis.
		{
			// Grab an object in the middle of the x-z plane. Use that to make a y-axis cast through the middle of the objects.
			const FObjectData& Object = Objects[GetIndex(CountX / 2, 0, CountZ / 2, CountX, CountY, CountZ)];
			const FVec3 Center = Object.Aabb.Center();

			FRaycastQueryData QueryData{ .Start = FVec3(Center.X, -10, Center.Z), .Direction = FVec3(0, 1, 0), .Length = RayLength };
			FRaycastQueryRuntimeData QueryRuntimeData(QueryData);
			BENCHMARK("Some Hits: Test y axis")
			{
				Visitor.Reset();
				SpatialPartition.Raycast(QueryRuntimeData, Visitor);
				return Visitor.Result;
			};
		}
		// Test a line along the z axis.
		{
			// Grab an object in the middle of the x-y plane. Use that to make a z-axis cast through the middle of the objects.
			const FObjectData& Object = Objects[GetIndex(CountX / 2, CountY / 2, 0, CountX, CountY, CountZ)];
			const FVec3 Center = Object.Aabb.Center();

			FRaycastQueryData QueryData{ .Start = FVec3(Center.X, Center.Y, -10), .Direction = FVec3(0, 0, 1), .Length = RayLength };
			FRaycastQueryRuntimeData QueryRuntimeData(QueryData);
			BENCHMARK("Some Hits: Test z axis")
			{
				Visitor.Reset();
				SpatialPartition.Raycast(QueryRuntimeData, Visitor);
				return Visitor.Result;
			};
		}
		// Test a line from one corner to the other. This should typically hit less than a single axis.
		{
			FVec3 Direction = Max - Min;
			Direction.Normalize();
			FRaycastQueryData QueryData{ .Start = Min - Direction, .Direction = Direction, .Length = RayLength };
			FRaycastQueryRuntimeData QueryRuntimeData(QueryData);
			BENCHMARK("Some Hits: Test x-y-z diagonal")
			{
				Visitor.Reset();
				SpatialPartition.Raycast(QueryRuntimeData, Visitor);
				return Visitor.Result;
			};
		}
	}

	template <typename SpatialPartitionType>
	void TestRaycastPerformanceFirstHit(SpatialPartitionType& SpatialPartition, const int32 CountX = 100, const int32 CountY = 100, int32 CountZ = 100)
	{
		TArray<FObjectData> Objects;
		BuildObjectList(Objects, CountX, CountY, CountZ);
		BuildFromObjects(SpatialPartition, Objects);

		const FVec3 Min = Objects[0].Aabb.Min();
		const FVec3 Max = Objects.Last().Aabb.Max();
		// Get an object in the middle. Note: Use this over the aabb center so we don't slice between two boxes depending on the input size.
		const FObjectData& Object = Objects[GetIndex(CountX / 2, CountY / 2, CountZ / 2, CountX, CountY, CountZ)];
		const FVec3 Center = Object.Aabb.GetCenter();
		FRaycastQueryData QueryData{ .Length = (Max - Min).Length() * 2 };

		TTestFindFirstObjectRaycastVisitor<FObjectData> Visitor(Objects);
		{
			QueryData.Start = FVec3(Min.X - 10, Center.Y, Center.Z);
			QueryData.Direction = FVec3(1, 0, 0);

			FRaycastQueryRuntimeData QueryRuntimeData(QueryData);
			BENCHMARK("Length Trimming: Positive X Axis")
			{
				Visitor.Reset();
				SpatialPartition.Raycast(QueryRuntimeData, Visitor);
				return Visitor.FirstHitUserData;
			};
		}
		{
			QueryData.Start = FVec3(Max.X + 1, Center.Y, Center.Z);
			QueryData.Direction = FVec3(-1, 0, 0);

			FRaycastQueryRuntimeData QueryRuntimeData(QueryData);
			BENCHMARK("Length Trimming: Negative X Axis")
			{
				Visitor.Reset();
				SpatialPartition.Raycast(QueryRuntimeData, Visitor);
				return Visitor.FirstHitUserData;
			};
		}
		{
			QueryData.Start = FVec3(Center.X, Min.Y - 1, Center.Z);
			QueryData.Direction = FVec3(0, 1, 0);

			FRaycastQueryRuntimeData QueryRuntimeData(QueryData);
			BENCHMARK("Length Trimming: Positive Y Axis")
			{
				Visitor.Reset();
				SpatialPartition.Raycast(QueryRuntimeData, Visitor);
				return Visitor.FirstHitUserData;
			};
		}
		{
			QueryData.Start = FVec3(Center.X, Max.Y + 1, Center.Z);
			QueryData.Direction = FVec3(0, -1, 0);

			FRaycastQueryRuntimeData QueryRuntimeData(QueryData);
			BENCHMARK("Length Trimming: Negative Y Axis")
			{
				Visitor.Reset();
				SpatialPartition.Raycast(QueryRuntimeData, Visitor);
				return Visitor.FirstHitUserData;
			};
		}
		{
			QueryData.Start = FVec3(Center.X, Center.Y, Min.Z - 1);
			QueryData.Direction = FVec3(0, 0, 1);

			FRaycastQueryRuntimeData QueryRuntimeData(QueryData);
			BENCHMARK("Length Trimming: Positive Z Axis")
			{
				Visitor.Reset();
				SpatialPartition.Raycast(QueryRuntimeData, Visitor);
				return Visitor.FirstHitUserData;
			};
		}
		{
			QueryData.Start = FVec3(Center.X, Center.Y, Max.Z + 1);
			QueryData.Direction = FVec3(0, 0, -1);

			FRaycastQueryRuntimeData QueryRuntimeData(QueryData);
			BENCHMARK("Length Trimming: Negative Z Axis")
			{
				Visitor.Reset();
				SpatialPartition.Raycast(QueryRuntimeData, Visitor);
				return Visitor.FirstHitUserData;
			};
		}
	}
} // namespace Chaos::SpatialPartition
