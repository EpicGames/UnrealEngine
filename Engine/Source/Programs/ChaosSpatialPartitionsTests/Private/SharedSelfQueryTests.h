// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "ChaosTestHarness.h"

#include "TestVisitors.h"

#include "ChaosSpatialPartitions/SpatialHandle.h"
#include "ObjectData.h"
#include "SharedHelpers.h"

namespace Chaos::SpatialPartition
{
	template <typename SpatialPartitionType>
	void TestSelfQueryBuildBasic(SpatialPartitionType& SpatialPartition)
	{
		// For basic build tests, we want to test:
		// - An empty spatial partition
		// - An empty result list
		// - Multiple results
		// Single result shouldn't really be necessary
		using FPair = FTestSelfQueryCollectorVisitor::FPair;
		const FVec3 SmallAabbSize = FVec3(1);
		const FVec3 LargeAabbSize = FVec3(2);
		const FVec3 Spacing(0.1f);
		TArray<FObjectData> Objects;
		BuildObjectList(Objects, 3, 3, 3, Spacing, SmallAabbSize);

		FObjectData& CenterObj = Objects[Objects.Num() / 2];

		SECTION("SelfQuery: Empty Spatial Partition")
		{
			FTestSelfQueryCollectorVisitor Visitor;
			SpatialPartition.SelfQuery(Visitor);

			CHECK(Visitor.Results.IsEmpty());
		}
		SECTION("SelfQuery: Empty Results")
		{
			BuildFromObjects(SpatialPartition, Objects);

			FTestSelfQueryCollectorVisitor Visitor;
			SpatialPartition.SelfQuery(Visitor);

			CHECK(Visitor.Results.IsEmpty());
		}
		SECTION("SelfQuery: Multiple Results")
		{
			// Update the center object to overlap everything.
			CenterObj.Aabb = BuildAabbCenterExtents(CenterObj.Aabb.GetCenter(), LargeAabbSize);
			BuildFromObjects(SpatialPartition, Objects);

			FTestSelfQueryCollectorVisitor Visitor;
			SpatialPartition.SelfQuery(Visitor);

			// We expect results where the center object overlaps every other object.
			TArray<FPair> Expected;
			for (const FObjectData& Obj : Objects)
			{
				if (Obj.UserData != CenterObj.UserData)
				{
					Expected.Add(FPair(Obj.UserData, CenterObj.UserData));
				}
			}
			CHECK(Visitor.Results == Expected);
		}
	}

	template <typename SpatialPartitionType>
	void TestSelfQueryInsert(SpatialPartitionType& SpatialPartition)
	{
		// For insertion, we want to test:
		// - Insert: No changes
		// - Insert: One new result
		// - Insert: Multiple new results
		using FPair = FTestSelfQueryCollectorVisitor::FPair;
		const FVec3 AabbSize(1);
		const FVec3 Spacing(0.1f);

		TArray<FObjectData> Objects;
		BuildObjectList(Objects, 3, 1, 1, Spacing, AabbSize);
		BuildFromObjects(SpatialPartition, Objects);

		const FObjectData& LastObject = Objects.Last();
		FObjectData NewObject;
		NewObject.UserData = LastObject.UserData + 1;

		SECTION("Insert: No Changes")
		{
			// Add a new object that doesn't overlap any of the other ones.
			NewObject.Aabb = LastObject.Aabb;
			NewObject.Aabb.MoveByVector(FVec3(3));
			InsertObject(SpatialPartition, NewObject);

			FTestSelfQueryCollectorVisitor Visitor;
			SpatialPartition.SelfQuery(Visitor);

			CHECK(Visitor.Results.IsEmpty());
		}
		SECTION("Insert: One New Result")
		{
			// Add a new object that overlaps the last object
			NewObject.Aabb = LastObject.Aabb;
			InsertObject(SpatialPartition, NewObject);

			FTestSelfQueryCollectorVisitor Visitor;
			SpatialPartition.SelfQuery(Visitor);

			const TArray<FPair> Expected
			{
				FPair(LastObject.UserData, NewObject.UserData),
			};
			CHECK(Visitor.Results == Expected);
		}
		SECTION("Insert: Multiple New Results")
		{
			// Add a new object that overlaps all of the objects
			NewObject.Aabb = FAABB3(Objects[0].Aabb.Min(), LastObject.Aabb.Max());
			InsertObject(SpatialPartition, NewObject);

			FTestSelfQueryCollectorVisitor Visitor;
			SpatialPartition.SelfQuery(Visitor);

			const TArray<FPair> Expected
			{
				FPair(Objects[0].UserData, NewObject.UserData),
				FPair(Objects[1].UserData, NewObject.UserData),
				FPair(Objects[2].UserData, NewObject.UserData),
			};
			CHECK(Visitor.Results == Expected);
		}
	}

	template <typename SpatialPartitionType>
	void TestSelfQueryRemove(SpatialPartitionType& SpatialPartition)
	{
		// For remove, we want to test:
		// - Remove: No changes
		// - Remove: One result removed
		// - Remove: Multiple results removed
		using FPair = FTestSelfQueryCollectorVisitor::FPair;
		const FVec3 AabbSize(1);
		const FVec3 Spacing(0.1f);

		TArray<FObjectData> Objects;
		BuildObjectList(Objects, 5, 1, 1, Spacing, AabbSize);
		// Make one object overlap the first 3 objects.
		Objects[3].Aabb = FAABB3(Objects[0].Aabb.Min(), Objects[2].Aabb.Max());
		// Make the last object overlap nothing.
		Objects[4].Aabb.MoveByVector(FVec3(10));

		SECTION("Remove: No Changes")
		{
			BuildFromObjects(SpatialPartition, Objects);

			// Remove the last object, this shouldn't have changed any results.
			RemoveObject(SpatialPartition, Objects[4]);

			FTestSelfQueryCollectorVisitor Visitor;
			SpatialPartition.SelfQuery(Visitor);

			const TArray<FPair> Expected
			{
				FPair(Objects[0].UserData, Objects[3].UserData),
				FPair(Objects[1].UserData, Objects[3].UserData),
				FPair(Objects[2].UserData, Objects[3].UserData),
			};
			CHECK(Visitor.Results == Expected);
		}
		SECTION("Remove: One Removed Result")
		{
			BuildFromObjects(SpatialPartition, Objects);

			// Remove object 0. This should only remove the pair (0, 3).
			RemoveObject(SpatialPartition, Objects[0]);

			FTestSelfQueryCollectorVisitor Visitor;
			SpatialPartition.SelfQuery(Visitor);

			const TArray<FPair> Expected
			{
				FPair(Objects[1].UserData, Objects[3].UserData),
				FPair(Objects[2].UserData, Objects[3].UserData),
			};
			CHECK(Visitor.Results == Expected);
		}
		SECTION("Remove: Multiple Removed Results")
		{
			BuildFromObjects(SpatialPartition, Objects);

			// Remove object 3. This should've removed all of the pairs.
			RemoveObject(SpatialPartition, Objects[3]);

			FTestSelfQueryCollectorVisitor Visitor;
			SpatialPartition.SelfQuery(Visitor);

			CHECK(Visitor.Results.IsEmpty());
		}
	}

	template <typename SpatialPartitionType>
	void TestSelfQueryUpdateBasic(SpatialPartitionType& SpatialPartition)
	{
		// For basic update tests, we need to test:
		// - No change
		// - One new result
		// - One removed result
		// - One new and one removed result
		// - Multiple changes
		using FPair = FTestSelfQueryCollectorVisitor::FPair;
		const FVec3 AabbSize(1);
		const FVec3 Spacing(0.1f);

		TArray<FObjectData> Objects;
		BuildObjectList(Objects, 5, 1, 1, Spacing, AabbSize);
		// Move the last object so it overlaps object 3,
		Objects[4].Aabb.MoveByVector(FVec3(-0.25f, 0, 0));

		SECTION("Update: No Change")
		{
			BuildFromObjects(SpatialPartition, Objects);

			// Move object 0. This still doesn't overlap anything so no results should change.
			Objects[0].Aabb.MoveByVector(FVec3(0, 5, 0));
			UpdateObject(SpatialPartition, Objects[0]);

			FTestSelfQueryCollectorVisitor Visitor;
			SpatialPartition.SelfQuery(Visitor);

			const TArray<FPair> Expected
			{
				FPair(Objects[3].UserData, Objects[4].UserData),
			};
			CHECK(Visitor.Results == Expected);
		}
		SECTION("Update: One New Result")
		{
			BuildFromObjects(SpatialPartition, Objects);

			// Move object 0 so it now overlaps object 1, adding 1 new pair.
			Objects[0].Aabb.MoveByVector(FVec3(0.25f, 0, 0));
			UpdateObject(SpatialPartition, Objects[0]);

			FTestSelfQueryCollectorVisitor Visitor;
			SpatialPartition.SelfQuery(Visitor);

			const TArray<FPair> Expected
			{
				FPair(Objects[0].UserData, Objects[1].UserData),
				FPair(Objects[3].UserData, Objects[4].UserData),
			};
			CHECK(Visitor.Results == Expected);
		}
		SECTION("Update: One Removed Result")
		{
			BuildFromObjects(SpatialPartition, Objects);

			// Move object 4 so it no longer overlaps 3, removing one pair.
			Objects[4].Aabb.MoveByVector(FVec3(0.25f, 0, 0));
			UpdateObject(SpatialPartition, Objects[4]);

			FTestSelfQueryCollectorVisitor Visitor;
			SpatialPartition.SelfQuery(Visitor);

			CHECK(Visitor.Results.IsEmpty());
		}
		SECTION("Update: One New And One Removed Result")
		{
			BuildFromObjects(SpatialPartition, Objects);

			// Move 4 so it no longer overlaps 3 but now overlaps 0.
			Objects[4].Aabb = Objects[0].Aabb;
			Objects[4].Aabb.MoveByVector(FVec3(-0.25f, 0, 0));
			UpdateObject(SpatialPartition, Objects[4]);

			FTestSelfQueryCollectorVisitor Visitor;
			SpatialPartition.SelfQuery(Visitor);

			const TArray<FPair> Expected
			{
				FPair(Objects[0].UserData, Objects[4].UserData),
			};
			CHECK(Visitor.Results == Expected);
		}
		SECTION("Update: Multiple Changes")
		{
			// Initialize 4 so it overlaps {2, 3}
			Objects[4].Aabb = FAABB3(Objects[2].Aabb.Min(), Objects[3].Aabb.Max());
			BuildFromObjects(SpatialPartition, Objects);

			// Move 4 so it now overlaps {0, 1}
			Objects[4].Aabb = FAABB3(Objects[0].Aabb.Min(), Objects[1].Aabb.Max());
			UpdateObject(SpatialPartition, Objects[4]);

			FTestSelfQueryCollectorVisitor Visitor;
			SpatialPartition.SelfQuery(Visitor);

			const TArray<FPair> Expected
			{
				FPair(Objects[0].UserData, Objects[4].UserData),
				FPair(Objects[1].UserData, Objects[4].UserData),
			};
			CHECK(Visitor.Results == Expected);
		}
	}

	template <typename SpatialPartitionType>
	void TestSelfQueryUpdateMovement(SpatialPartitionType& SpatialPartition)
	{
		// In general, we want to test a few scenarios that simulate an object moving:
		// 1. A small update along each axis, both in the positive and negative directions.
		// 2. A large update along each axis. This is useful to make sure self pairs aren't created.
		//    - An example of this is SAP where moving far enough that a min crosses a max (or vice-versa) could cause a self pair to get created.
		// 3. Growing on all axes (shrinking is handled separately).
		using FPair = FTestSelfQueryCollectorVisitor::FPair;
		const FVec3 Spacing(0.5f);
		const FVec3 AabbSize(1, 1, 1);

		TArray<FObjectData> Objects;
		BuildObjectList(Objects, 3, 3, 3, Spacing, AabbSize);
		BuildFromObjects(SpatialPartition, Objects);

		// Grab the middle object, we'll use this for testing
		FObjectData& TestObject = Objects[GetIndex(1, 1, 1, 3, 3, 3)];

		// Test Moving along each axis with a small positive / negative value.
		for (int32 Axis = 0; Axis < 3; ++Axis)
		{
			for (int32 AxisDirection = -1; AxisDirection <= 1; AxisDirection += 2)
			{
				DYNAMIC_SECTION("Move Small Amount Along Axis " << Axis << " by direction " << AxisDirection)
				{
					// Offset the center object by a small amount so it now overlaps the neighbor on the axis.
					FVec3 Offset = FVec3::Zero();
					Offset[Axis] = AxisDirection * AabbSize[Axis] * 0.75f;
					TestObject.Aabb.MoveByVector(Offset);
					UpdateObject(SpatialPartition, TestObject);

					int32 NeighborIndex[3] = { 1, 1, 1 };
					NeighborIndex[Axis] += AxisDirection;
					const int32 ExpectedHitIndex = GetIndex(NeighborIndex[0], NeighborIndex[1], NeighborIndex[2], 3, 3, 3);
					const TArray<FPair> Expected
					{
						FPair(Objects[ExpectedHitIndex].UserData, TestObject.UserData),
					};

					FTestSelfQueryCollectorVisitor Visitor;
					SpatialPartition.SelfQuery(Visitor);
					CHECK(Visitor.Results == Expected);
				}
			}
		}

		// Test Moving along each axis with a large positive / negative value (large enough for the object to not intersect its old position).
		for (int32 Axis = 0; Axis < 3; ++Axis)
		{
			for (int32 AxisDirection = -1; AxisDirection <= 1; AxisDirection += 2)
			{
				DYNAMIC_SECTION("Move Large Amount Along Axis " << Axis << " by direction " << AxisDirection)
				{
					// Move the center object by a large amount (more than it's full size), but still have it overlap its neighbor.
					FVec3 Offset = FVec3::Zero();
					Offset[Axis] = AxisDirection * AabbSize[Axis] * 2;
					TestObject.Aabb.MoveByVector(Offset);
					UpdateObject(SpatialPartition, TestObject);

					int32 NeighborIndex[3] = { 1, 1, 1 };
					NeighborIndex[Axis] += AxisDirection;
					const int32 ExpectedHitIndex = GetIndex(NeighborIndex[0], NeighborIndex[1], NeighborIndex[2], 3, 3, 3);
					const TArray<FPair> Expected
					{
						FPair(Objects[ExpectedHitIndex].UserData, TestObject.UserData),
					};

					FTestSelfQueryCollectorVisitor Visitor;
					SpatialPartition.SelfQuery(Visitor);
					CHECK(Visitor.Results == Expected);
				}
			}
		}

		SECTION("Size Change Aabb")
		{
			// Grow the center object's aabb so it overlaps all neighbors
			TestObject.Aabb = BuildAabbCenterExtents(TestObject.Aabb.GetCenter(), FVec3(2));
			UpdateObject(SpatialPartition, TestObject);

			SECTION("Grow Aabb")
			{
				TArray<FPair> Expected;
				for (const FObjectData& Obj : Objects)
				{
					if (Obj.UserData != TestObject.UserData)
					{
						Expected.Add(FPair(Obj.UserData, TestObject.UserData));
					}
				}

				FTestSelfQueryCollectorVisitor Visitor;
				SpatialPartition.SelfQuery(Visitor);
				CHECK(Visitor.Results == Expected);
			}
		}
	}

	template <typename SpatialPartitionType>
	void TestSelfQueryUpdateWithinBounds(SpatialPartitionType& SpatialPartition)
	{
		// Build the objects with initial overlap, but then shrink the center object so it doesn't touch any neighbor.
		const FVec3 Spacing(-0.1f);
		TArray<FObjectData> Objects;
		BuildObjectList(Objects, 3, 1, 1, Spacing);
		BuildFromObjects(SpatialPartition, Objects);

		Objects[1].Aabb.ShrinkSymmetrically(FVec3(0.25f));
		UpdateObject(SpatialPartition, Objects[1]);

		FTestSelfQueryCollectorVisitor Visitor;
		SpatialPartition.SelfQuery(Visitor);
		CHECK(Visitor.Results.IsEmpty());
	}

	template <typename SpatialPartitionType>
	void TestSelfQueryPerformanceEmpty(SpatialPartitionType& SpatialPartition)
	{
		BENCHMARK("Self Query: Empty")
		{
			FTestSelfQueryAccumulatorVisitor Visitor;
			SpatialPartition.SelfQuery(Visitor);
			return Visitor.Result;
		};
	}

	template <typename SpatialPartitionType>
	void TestSelfQueryPerformanceBasic(SpatialPartitionType& SpatialPartition, const int32 CountX = 100, const int32 CountY = 100, int32 CountZ = 100)
	{
		// Make the aabbs slightly larger so all neighbors overlap
		const FVec3 AabbSize(1.05f);

		TArray<FObjectData> Objects;
		BuildObjectList(Objects, CountX, CountY, CountZ, FVec3::Zero(), AabbSize);
		BuildFromObjects(SpatialPartition, Objects);

		BENCHMARK("Self Query: Basic")
		{
			FTestSelfQueryAccumulatorVisitor Visitor;
			SpatialPartition.SelfQuery(Visitor);
			return Visitor.Result;
		};
	}
} // namespace Chaos::SpatialPartition
