// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "ChaosTestHarness.h"

#include "TestVisitors.h"

#include "ChaosSpatialPartitions/SpatialHandle.h"
#include "ObjectData.h"
#include "SharedHelpers.h"

namespace Chaos::SpatialPartition
{
	template <typename SpatialPartitionType>
	void TestSweepBuildBasic(SpatialPartitionType& SpatialPartition)
	{
		// For basic build tests, we want to test:
		// - An empty spatial partition
		// - An empty result list
		// - Single object (each object)
		// - Multiple results

		const FVec3 AabbSize = FVec3(1);
		constexpr int32 Count = 4;
		// Add a spacing of 1 between to make it so we can do easy queries that only hit 1 object.
		const FVec3 Spacing(1);
		TArray<FObjectData> Objects;
		BuildObjectList(Objects, Count, 1, 1, Spacing, AabbSize);

		FSweepQueryData QueryData;
		QueryData.Start = FVec3::Zero();
		QueryData.Length = 10;
		QueryData.Direction = FVec3(0, -1, 0);
		QueryData.HalfExtents = FVec3(0.5f);

		FTestSweepCollectorVisitor Visitor;
		SECTION("Sweep: Empty Spatial Partition")
		{
			FSweepQueryRuntimeData QueryRuntimeData(QueryData);
			EVisitResult VisitResult = SpatialPartition.Sweep(QueryRuntimeData, Visitor);

			CHECK(VisitResult == EVisitResult::Continue);
			CHECK(Visitor.Results.IsEmpty());
		}

		BuildFromObjects(SpatialPartition, Objects);
		SECTION("Sweep: Hit None")
		{
			QueryData.Start = FVec3(-1, 2, 0.5f);
			QueryData.Direction = FVec3(1, 0, 0);

			FSweepQueryRuntimeData QueryRuntimeData(QueryData);
			const EVisitResult VisitResult = SpatialPartition.Sweep(QueryRuntimeData, Visitor);

			CHECK(VisitResult == EVisitResult::Continue);
			CHECK(Visitor.Results.IsEmpty());
		}
		for (int32 Index = 0; Index < Count; ++Index)
		{
			DYNAMIC_SECTION("Sweep: Hit Object " << Index)
			{
				const FObjectData& TestObject = Objects[Index];
				// Set the sweep start half-way back along the cast direction
				QueryData.Start = TestObject.Aabb.Center() - QueryData.Direction * QueryData.Length * 0.5f;

				FSweepQueryRuntimeData QueryRuntimeData(QueryData);
				const EVisitResult VisitResult = SpatialPartition.Sweep(QueryRuntimeData, Visitor);

				CHECK(VisitResult == EVisitResult::Continue);
				CHECK(Visitor.Results == TArray<FUserDataType>{TestObject.UserData});
			}
		}
		// Test a sweep half-way in between two objects but with enough thickness to hit both.
		for (int32 Index = 1; Index < Count; ++Index)
		{
			DYNAMIC_SECTION("Sweep: Hit Objects [" << Index - 1 << "," << Index << "]")
			{
				const FObjectData& TestObject0 = Objects[Index - 1];
				const FObjectData& TestObject1 = Objects[Index];
				const FVec3 MidPoint = (TestObject0.Aabb.Center() + TestObject1.Aabb.Center()) * 0.5f;
				QueryData.HalfExtents = FVec3(1.5f);
				QueryData.Start = MidPoint - QueryData.Direction * QueryData.Length * 0.5f;

				FSweepQueryRuntimeData QueryRuntimeData(QueryData);
				const EVisitResult VisitResult = SpatialPartition.Sweep(QueryRuntimeData, Visitor);

				CHECK(VisitResult == EVisitResult::Continue);
				CHECK(Visitor.Results == TArray<FUserDataType>{TestObject0.UserData, TestObject1.UserData});
			}
		}
		SECTION("Sweep: Hit All")
		{
			TArray<FUserDataType> Expected;
			for (const FObjectData& Object : Objects)
			{
				Expected.Add(Object.UserData);
			}

			QueryData.Start = FVec3(-1, 0.5f, 0.5f);
			QueryData.Direction = FVec3(1, 0, 0);

			FSweepQueryRuntimeData QueryRuntimeData(QueryData);
			const EVisitResult VisitResult = SpatialPartition.Sweep(QueryRuntimeData, Visitor);

			CHECK(VisitResult == EVisitResult::Continue);
			CHECK(Visitor.Results == Expected);
		}
	}

	template <typename SpatialPartitionType>
	void TestSweepInsert(SpatialPartitionType& SpatialPartition)
	{
		constexpr int32 Count = 4;
		const FVec3 Spacing(1);
		TArray<FObjectData> Objects;
		BuildObjectList(Objects, Count, 1, 1, Spacing);
		// Move object 0 so it's behind the sweep
		Objects[0].Aabb.MoveByVector(FVec3(-5, 0, 0));
		// Move object 2 up so it's just outside the half-extents
		Objects[2].Aabb.MoveByVector(FVec3(0, 0.6f, 0));

		FSweepQueryData QueryData;
		QueryData.Start = FVec3(0, 0, 0);
		QueryData.Length = 10;
		QueryData.Direction = FVec3(1, 0, 0);
		QueryData.HalfExtents = FVec3(0.5f);

		FSweepQueryRuntimeData QueryRuntimeData(QueryData);
		FTestSweepCollectorVisitor Visitor;

		InsertObject(SpatialPartition, Objects[0]);
		Visitor.Reset();
		CHECK(EVisitResult::Continue == SpatialPartition.Sweep(QueryRuntimeData, Visitor));
		CHECK(Visitor.Results.IsEmpty());

		InsertObject(SpatialPartition, Objects[1]);
		Visitor.Reset();
		CHECK(EVisitResult::Continue == SpatialPartition.Sweep(QueryRuntimeData, Visitor));
		CHECK(Visitor.Results == TArray<FUserDataType>{Objects[1].UserData});

		InsertObject(SpatialPartition, Objects[2]);
		Visitor.Reset();
		CHECK(EVisitResult::Continue == SpatialPartition.Sweep(QueryRuntimeData, Visitor));
		CHECK(Visitor.Results == TArray<FUserDataType>{Objects[1].UserData});

		InsertObject(SpatialPartition, Objects[3]);
		Visitor.Reset();
		CHECK(EVisitResult::Continue == SpatialPartition.Sweep(QueryRuntimeData, Visitor));
		CHECK(Visitor.Results == TArray<FUserDataType>{Objects[1].UserData, Objects[3].UserData});
	}

	template <typename SpatialPartitionType>
	void TestSweepRemove(SpatialPartitionType& SpatialPartition)
	{
		constexpr int32 Count = 4;
		const FVec3 Spacing(1);
		TArray<FObjectData> Objects;
		BuildObjectList(Objects, Count, 1, 1, Spacing);
		// Move object 0 so it's behind the sweep
		Objects[0].Aabb.MoveByVector(FVec3(-5, 0, 0));
		// Move object 2 up so it's just outside the half-extents
		Objects[2].Aabb.MoveByVector(FVec3(0, 0.6f, 0));
		BuildFromObjects(SpatialPartition, Objects);

		FSweepQueryData QueryData;
		QueryData.Start = FVec3(0, 0, 0);
		QueryData.Length = 10;
		QueryData.Direction = FVec3(1, 0, 0);
		QueryData.HalfExtents = FVec3(0.5f);

		FSweepQueryRuntimeData QueryRuntimeData(QueryData);
		FTestSweepCollectorVisitor Visitor;

		RemoveObject(SpatialPartition, Objects[3]);
		Visitor.Reset();
		CHECK(EVisitResult::Continue == SpatialPartition.Sweep(QueryRuntimeData, Visitor));
		CHECK(Visitor.Results == TArray<FUserDataType>{Objects[1].UserData});

		RemoveObject(SpatialPartition, Objects[2]);
		Visitor.Reset();
		CHECK(EVisitResult::Continue == SpatialPartition.Sweep(QueryRuntimeData, Visitor));
		CHECK(Visitor.Results == TArray<FUserDataType>{Objects[1].UserData});

		RemoveObject(SpatialPartition, Objects[1]);
		Visitor.Reset();
		CHECK(EVisitResult::Continue == SpatialPartition.Sweep(QueryRuntimeData, Visitor));
		CHECK(Visitor.Results.IsEmpty());

		RemoveObject(SpatialPartition, Objects[0]);
		Visitor.Reset();
		CHECK(EVisitResult::Continue == SpatialPartition.Sweep(QueryRuntimeData, Visitor));
		CHECK(Visitor.Results.IsEmpty());
	}

	template <typename SpatialPartitionType>
	void TestSweepUpdateBasic(SpatialPartitionType& SpatialPartition)
	{
		// This test starts with all objects in a region and moves half of them to a new region. 
		// This tests both the before and after locations, and does so before and after the moves.
		constexpr int32 Count = 4;
		const FVec3 Spacing(1);
		TArray<FObjectData> Objects;
		BuildObjectList(Objects, Count, 1, 1, Spacing);
		BuildFromObjects(SpatialPartition, Objects);

		const FVec3 Offset(0, 5, 0);
		FSweepQueryData OldQueryData{ .Start = FVec3(-1, 0.5f, 0.5f), .Direction = FVec3(1, 0, 0), .HalfExtents = FVec3(0.5f), .Length = 20 };
		FSweepQueryData NewQueryData{ .Start = OldQueryData.Start + Offset, .Direction = FVec3(1, 0, 0), .HalfExtents = FVec3(0.5f), .Length = 20 };
		FSweepQueryRuntimeData OldQueryRuntimeData(OldQueryData);
		FSweepQueryRuntimeData NewQueryRuntimeData(NewQueryData);
		FTestSweepCollectorVisitor Visitor;

		// First verify the original query results. We expect all of the objects in the "old" location and none in the "new"
		Visitor.Reset();
		CHECK(EVisitResult::Continue == SpatialPartition.Sweep(OldQueryRuntimeData, Visitor));
		CHECK(Visitor.Results == TArray<FUserDataType>{Objects[0].UserData, Objects[1].UserData, Objects[2].UserData, Objects[3].UserData});

		Visitor.Reset();
		CHECK(EVisitResult::Continue == SpatialPartition.Sweep(NewQueryRuntimeData, Visitor));
		CHECK(Visitor.Results.IsEmpty());

		// Now move some of the objects and re-test.
		// Object 0 is moved behind the sweep while object 2 is moved into the new area.
		Objects[0].Aabb.MoveByVector(FVec3(-10, 0, 0));
		Objects[2].Aabb.MoveByVector(Offset);
		UpdateObject(SpatialPartition, Objects[0]);
		UpdateObject(SpatialPartition, Objects[2]);

		Visitor.Reset();
		CHECK(EVisitResult::Continue == SpatialPartition.Sweep(NewQueryRuntimeData, Visitor));
		CHECK(Visitor.Results == TArray<FUserDataType>{Objects[2].UserData});

		Visitor.Reset();
		CHECK(EVisitResult::Continue == SpatialPartition.Sweep(OldQueryRuntimeData, Visitor));
		CHECK(Visitor.Results == TArray<FUserDataType>{Objects[1].UserData, Objects[3].UserData});
	}

	template <typename SpatialPartitionType>
	void TestSweepUpdateWithinBounds(SpatialPartitionType& SpatialPartition)
	{
		constexpr int32 Count = 4;
		const FVec3 Spacing(1);

		TArray<FObjectData> Objects;
		BuildObjectList(Objects, Count, 1, 1, Spacing);
		BuildFromObjects(SpatialPartition, Objects);

		FObjectData& TestObject = Objects.Last();
		const FVector AabbSize = TestObject.Aabb.Extents();
		const FVec3 AabbMin = TestObject.Aabb.Min();
		const FVec3 AabbMax = TestObject.Aabb.Max();
		const FVec3 SmallSize = AabbSize / 8;

		// Update the aabb to be 1/8 the size
		TestObject.Aabb = FAABB3(AabbMin, AabbMin + SmallSize);
		UpdateObject(SpatialPartition, TestObject);

		FTestSweepCollectorVisitor Visitor;
		FSweepQueryData QueryData{ .Direction = FVec3(0, 0, -1), .HalfExtents = SmallSize, .Length = 10 };

		// Test the new bounds to make sure it's valid
		QueryData.Start = TestObject.Aabb.GetCenter() - QueryData.Direction * 5;
		FSweepQueryRuntimeData NewQueryData(QueryData);
		CHECK(EVisitResult::Continue == SpatialPartition.Sweep(NewQueryData, Visitor));
		CHECK(Visitor.Results == TArray<FUserDataType>{TestObject.UserData});

		// Test something within the old bounds but not hitting the new bounds.
		Visitor.Results.Reset();
		QueryData.Start = AabbMax - SmallSize - QueryData.Direction * 5;
		FSweepQueryRuntimeData OldQueryData(QueryData);
		CHECK(EVisitResult::Continue == SpatialPartition.Sweep(OldQueryData, Visitor));
		CHECK(Visitor.Results.IsEmpty());
	}

	template <typename SpatialPartitionType>
	void TestSweepWithEarlyVisitResultTermination(SpatialPartitionType& SpatialPartition)
	{
		constexpr int32 Count = 4;

		TArray<FObjectData> Objects;
		BuildObjectList(Objects, Count);
		BuildFromObjects(SpatialPartition, Objects);

		// Build a sweep that hits all of the objects, but specify to terminate after 2 visits.
		FTestSweepCollectorVisitor Visitor;
		Visitor.MaxResults = 2;
		FSweepQueryData QueryData{ .Start = FVec3(-5, 0.5f, 0.5f), .Direction = FVec3(1, 0, 0), .HalfExtents = FVec3(0.1f), .Length = 100 };
		FSweepQueryRuntimeData QueryRuntimeData(QueryData);
		const EVisitResult VisitResult = SpatialPartition.Sweep(QueryRuntimeData, Visitor);

		CHECK(VisitResult == EVisitResult::Stop);
		CHECK(Visitor.Results.Num() == 2);
	}

	template <typename SpatialPartitionType>
	void TestSweepWithEarlyLengthTermination(SpatialPartitionType& SpatialPartition)
	{
		constexpr int32 Count = 4;

		TArray<FObjectData> Objects;
		BuildObjectList(Objects, Count);
		BuildFromObjects(SpatialPartition, Objects);

		// Build a sweep that hits all of the objects, but change the length to something really small after the first hit. 
		// This should cause all other sweeps to fail.
		FTestSweepCollectorVisitor Visitor;
		Visitor.TargetLength = 0.1f;
		Visitor.bUpdateLength = true;
		FSweepQueryData QueryData{ .Start = FVec3(-5, 0.5f, 0.5f), .Direction = FVec3(1, 0, 0), .HalfExtents = FVec3(0.1f), .Length = 100 };
		FSweepQueryRuntimeData QueryRuntimeData(QueryData);
		const EVisitResult VisitResult = SpatialPartition.Sweep(QueryRuntimeData, Visitor);

		CHECK(VisitResult == EVisitResult::Continue);
		CHECK(Visitor.Results.Num() == 1);
	}
	template <typename SpatialPartitionType>
	void TestSweepPerformanceEmpty(SpatialPartitionType& SpatialPartition)
	{
		FSweepQueryData QueryData{ .Start = FVec3::Zero(), .Direction = FVec3(1, 0, 0), .HalfExtents = FVec3(0.5f), .Length = 10 };
		FSweepQueryRuntimeData QueryRuntimeData(QueryData);
		FTestSweepAccumulatorVisitor Visitor;
		BENCHMARK("Sweep: Empty")
		{
			SpatialPartition.Sweep(QueryRuntimeData, Visitor);
			return Visitor.Result;
		};
	}
	template <typename SpatialPartitionType>
	void TestSweepPerformanceBasic(SpatialPartitionType& SpatialPartition, const int32 CountX = 10000, const int32 CountY = 10, const int32 CountZ = 2)
	{
		const FVec3 Spacing(1);
		const FVec3 AabbSize(1, 1, 1);

		TArray<FObjectData> Objects;
		BuildObjectList(Objects, CountX, CountY, CountZ, Spacing);
		const FVec3 Min = Objects[0].Aabb.Min();
		const FVec3 Max = Objects.Last().Aabb.Max();
		const FReal RayLength = (Max - Min).Length() * 2;

		BuildFromObjects(SpatialPartition, Objects);

		FTestSweepAccumulatorVisitor Visitor;
		FSweepQueryData QueryData{ .HalfExtents = FVec3(0.1f), .Length = RayLength };

		// Test having no overlaps
		{
			// Test a sweep outside the root and pointing away from everything. 
			// For hierarchical spatial partitions this should terminate very early.
			{
				QueryData.Start = FVec3(-10, -10, -10);
				QueryData.Direction = FVec3(0, 0, -1);
				FSweepQueryRuntimeData QueryRuntimeData(QueryData);

				BENCHMARK("No Hits: Far away query")
				{
					Visitor.Reset();
					SpatialPartition.Sweep(QueryRuntimeData, Visitor);
					return Visitor.Result;
				};
			}
			// Test a query in-between a bunch of objects. 
			// This helps to test hierarchical spatial partitions as a good chunk of the hierarchy will have to be traversed.
			{
				// Grab an object in the middle of the y-z plane and offset on the y-axis half-way in between it and the next object.
				// Note: This assumes CountY and CountZ are greater than 1, otherwise it'll miss the root.
				const FObjectData& Object = Objects[GetIndex(0, CountY / 2, CountZ / 2, CountX, CountY, CountZ)];
				FVec3 Start = Object.Aabb.Center();
				Start.Y += (AabbSize.Y + Spacing.Y) / 2.0f;

				QueryData.Start = FVec3(-10, Start.Y, Start.Z);
				QueryData.Direction = FVec3(1, 0, 0);
				FSweepQueryRuntimeData QueryRuntimeData(QueryData);
				BENCHMARK("No Hits: Query between objects")
				{
					Visitor.Reset();
					SpatialPartition.Sweep(QueryRuntimeData, Visitor);
					return Visitor.Result;
				};
			}
		}
		// Test a line along the x axis.
		{
			// Grab an object in the middle of the y-z plane. Use that to make a x-axis cast through the middle of the objects.
			const FObjectData& Object = Objects[GetIndex(0, CountY / 2, CountZ / 2, CountX, CountY, CountZ)];
			const FVec3 Center = Object.Aabb.Center();

			QueryData.Start = FVec3(-10, Center.Y, Center.Z);
			QueryData.Direction = FVec3(1, 0, 0);
			FSweepQueryRuntimeData QueryRuntimeData(QueryData);
			BENCHMARK("Some Hits: Test x axis")
			{
				Visitor.Reset();
				SpatialPartition.Sweep(QueryRuntimeData, Visitor);
				return Visitor.Result;
			};
		}
		// Test a line along the y axis.
		{
			// Grab an object in the middle of the x-z plane. Use that to make a y-axis cast through the middle of the objects.
			const FObjectData& Object = Objects[GetIndex(CountX / 2, 0, CountZ / 2, CountX, CountY, CountZ)];
			const FVec3 Center = Object.Aabb.Center();

			QueryData.Start = FVec3(Center.X, -10, Center.Z);
			QueryData.Direction = FVec3(0, 1, 0);
			FSweepQueryRuntimeData QueryRuntimeData(QueryData);
			BENCHMARK("Some Hits: Test y axis")
			{
				Visitor.Reset();
				SpatialPartition.Sweep(QueryRuntimeData, Visitor);
				return Visitor.Result;
			};
		}
		// Test a line along the z axis.
		{
			// Grab an object in the middle of the x-y plane. Use that to make a z-axis cast through the middle of the objects.
			const FObjectData& Object = Objects[GetIndex(CountX / 2, CountY / 2, 0, CountX, CountY, CountZ)];
			const FVec3 Center = Object.Aabb.Center();

			QueryData.Start = FVec3(Center.X, Center.Y, -10);
			QueryData.Direction = FVec3(0, 0, 1);
			FSweepQueryRuntimeData QueryRuntimeData(QueryData);
			BENCHMARK("Some Hits: Test z axis")
			{
				Visitor.Reset();
				SpatialPartition.Sweep(QueryRuntimeData, Visitor);
				return Visitor.Result;
			};
		}
		// Test a line from one corner to the other. This should typically hit less than a single axis.
		{
			QueryData.Direction = Max - Min;
			QueryData.Direction.Normalize();
			QueryData.Start = Min - QueryData.Direction;
			FSweepQueryRuntimeData QueryRuntimeData(QueryData);
			BENCHMARK("Some Hits: Test x-y-z diagonal")
			{
				Visitor.Reset();
				SpatialPartition.Sweep(QueryRuntimeData, Visitor);
				return Visitor.Result;
			};
		}
	}

	template <typename SpatialPartitionType>
	void TestSweepPerformanceFirstHit(SpatialPartitionType& SpatialPartition, const int32 CountX = 50, const int32 CountY = 50, int32 CountZ = 50)
	{
		const FVec3 Spacing(1);
		TArray<FObjectData> Objects;
		BuildObjectList(Objects, CountX, CountY, CountZ, Spacing);
		BuildFromObjects(SpatialPartition, Objects);

		const FVec3 Min = Objects[0].Aabb.Min();
		const FVec3 Max = Objects.Last().Aabb.Max();
		// Get an object in the middle. Note: Use this over the aabb center so we don't slice between two boxes depending on the input size.
		const FObjectData& Object = Objects[GetIndex(CountX / 2, CountY / 2, CountZ / 2, CountX, CountY, CountZ)];
		const FVec3 Center = Object.Aabb.GetCenter();
		FSweepQueryData QueryData{ .HalfExtents = FVec3(0.5f), .Length = (Max - Min).Length() * 2 };

		TTestFindFirstObjectSweepVisitor<FObjectData> Visitor(Objects);
		{
			QueryData.Start = FVec3(Min.X - 10, Center.Y, Center.Z);
			QueryData.Direction = FVec3(1, 0, 0);

			FSweepQueryRuntimeData QueryRuntimeData(QueryData);
			BENCHMARK("Length Trimming: Positive X Axis")
			{
				Visitor.Reset();
				SpatialPartition.Sweep(QueryRuntimeData, Visitor);
				return Visitor.FirstHitUserData;
			};
		}
		{
			QueryData.Start = FVec3(Max.X + 1, Center.Y, Center.Z);
			QueryData.Direction = FVec3(-1, 0, 0);

			FSweepQueryRuntimeData QueryRuntimeData(QueryData);
			BENCHMARK("Length Trimming: Negative X Axis")
			{
				Visitor.Reset();
				SpatialPartition.Sweep(QueryRuntimeData, Visitor);
				return Visitor.FirstHitUserData;
			};
		}
		{
			QueryData.Start = FVec3(Center.X, Min.Y - 1, Center.Z);
			QueryData.Direction = FVec3(0, 1, 0);

			FSweepQueryRuntimeData QueryRuntimeData(QueryData);
			BENCHMARK("Length Trimming: Positive Y Axis")
			{
				Visitor.Reset();
				SpatialPartition.Sweep(QueryRuntimeData, Visitor);
				return Visitor.FirstHitUserData;
			};
		}
		{
			QueryData.Start = FVec3(Center.X, Max.Y + 1, Center.Z);
			QueryData.Direction = FVec3(0, -1, 0);

			FSweepQueryRuntimeData QueryRuntimeData(QueryData);
			BENCHMARK("Length Trimming: Negative Y Axis")
			{
				Visitor.Reset();
				SpatialPartition.Sweep(QueryRuntimeData, Visitor);
				return Visitor.FirstHitUserData;
			};
		}
		{
			QueryData.Start = FVec3(Center.X, Center.Y, Min.Z - 1);
			QueryData.Direction = FVec3(0, 0, 1);

			FSweepQueryRuntimeData QueryRuntimeData(QueryData);
			BENCHMARK("Length Trimming: Positive Z Axis")
			{
				Visitor.Reset();
				SpatialPartition.Sweep(QueryRuntimeData, Visitor);
				return Visitor.FirstHitUserData;
			};
		}
		{
			QueryData.Start = FVec3(Center.X, Center.Y, Max.Z + 1);
			QueryData.Direction = FVec3(0, 0, -1);

			FSweepQueryRuntimeData QueryRuntimeData(QueryData);
			BENCHMARK("Length Trimming: Negative Z Axis")
			{
				Visitor.Reset();
				SpatialPartition.Sweep(QueryRuntimeData, Visitor);
				return Visitor.FirstHitUserData;
			};
		}
	}
} // namespace Chaos::SpatialPartition
