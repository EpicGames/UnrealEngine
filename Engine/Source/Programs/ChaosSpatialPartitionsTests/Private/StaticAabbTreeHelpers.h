// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ChaosSpatialPartitions/Common.h"

#include "ChaosSpatialPartitions/Library/AabbTreeLeaf.h"
#include "ChaosSpatialPartitions/Library/AabbTreeNode.h"

namespace Chaos::SpatialPartition
{
	inline TArray<FAabbTreeLeafElement> BuildElementsFromAabbs(const TArray<FAABB3>& Aabbs)
	{
		TArray<FAabbTreeLeafElement> Elements;
		Elements.Reserve(Aabbs.Num());
		for (int32 I = 0; I < Aabbs.Num(); ++I)
		{
			Elements.Emplace(FAabbTreeLeafElement{ .Aabb = Aabbs[I], .Index = I });
		}
		return Elements;
	}

	// Returns the index of the first leaf where there's an element with the given search value.
	// This is used in tests to find where some input data ended up in the final tree.
	inline int32 FindLeafWithElementIndexValue(const TArray<FAabbTreeLeaf>& Leaves, const int32 SearchValue)
	{
		for (int32 LeafIndex = 0; LeafIndex < Leaves.Num(); ++LeafIndex)
		{
			const FAabbTreeLeaf& Leaf = Leaves[LeafIndex];
			for (const FAabbTreeLeafElement& Element : Leaf.Elements)
			{
				if (Element.Index == SearchValue)
				{
					return LeafIndex;
				}
			}
		}
		return INDEX_NONE;
	}

	inline const FAabbTreeLeaf* FindLeafPointerWithHandleValue(const TArray<FAabbTreeLeaf>& Leaves, const int32 SearchValue)
	{
		const int32 LeafIndex = FindLeafWithElementIndexValue(Leaves, SearchValue);
		return LeafIndex == INDEX_NONE ? nullptr : &Leaves[LeafIndex];
	}

	inline void FindLeafAndNodePointers(const TArray<FAabbTreeLeaf>& Leaves, const TArray<FAabbTreeNode>& Nodes, const int32 IndexCount, TArray<const FAabbTreeLeaf*>& OutLeaves, TArray<const FAabbTreeNode*>& OutNodes)
	{
		OutLeaves.SetNum(IndexCount);
		OutNodes.SetNum(IndexCount);
		for (int32 I = 0; I < IndexCount; ++I)
		{
			OutLeaves[I] = FindLeafPointerWithHandleValue(Leaves, I);
			const int32 NodeIndex = (OutLeaves[I] != nullptr) ? OutLeaves[I]->NodeIndex : INDEX_NONE;
			OutNodes[I] = Nodes.IsValidIndex(NodeIndex) ? &Nodes[NodeIndex] : nullptr;
		}
	}
} // namespace Chaos::SpatialPartition
