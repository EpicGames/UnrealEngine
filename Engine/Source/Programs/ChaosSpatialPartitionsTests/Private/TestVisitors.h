// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include <ostream>

#include "ChaosSpatialPartitions/Visitors.h"

namespace Chaos::SpatialPartition
{
	// Visitor that accumulates the user data. This is primarily used for perf tests to guarantee the results are used.
	template <typename QueryDataType, template <typename> typename AdapterType>
	struct TTestAccumulatorVisitor : public AdapterType<TTestAccumulatorVisitor<QueryDataType, AdapterType>>
	{
		EVisitResult Visit(const FUserDataType& UserData, QueryDataType& QueryData)
		{
			Result += UserData;
			return EVisitResult::Continue;
		}

		void Reset()
		{
			Result = 0;
		}

		int32 Result = 0;
	};

	using FTestOverlapAccumulatorVisitor = TTestAccumulatorVisitor<FOverlapQueryRuntimeData, TOverlapVisitorAdapter>;
	using FTestRaycastAccumulatorVisitor = TTestAccumulatorVisitor<FRaycastQueryRuntimeData, TRaycastVisitorAdapter>;
	using FTestSweepAccumulatorVisitor = TTestAccumulatorVisitor<FSweepQueryRuntimeData, TSweepVisitorAdapter>;

	// Visitor that accumulates the user data. This is primarily used for perf tests to guarantee the results are used.
	struct FTestSelfQueryAccumulatorVisitor : public TSelfQueryVisitorAdapter<FTestSelfQueryAccumulatorVisitor>
	{
		void Visit(const FUserDataType& UserData0, const FUserDataType& UserData1)
		{
			Result += UserData0 + UserData1;
		}

		void Reset()
		{
			Result = 0;
		}

		int32 Result = 0;
	};

	// Visitor that collects all results from visit calls. This is used to check all expected results are returned. 
	// As traversal order is not guaranteed, this sorts the results to ensure consistent and stable tests.
	template <typename QueryDataType, template <typename> typename AdapterType>
	struct TTestCollectorVisitor : public AdapterType<TTestCollectorVisitor<QueryDataType, AdapterType>>
	{
		EVisitResult Visit(const FUserDataType& UserData, QueryDataType& QueryData)
		{
			Results.Add(UserData);
			Results.Sort();
			// Update the length if requests. Note: This is only valid for raycast/sweeps.
			if constexpr (std::is_same_v<QueryDataType, FRaycastQueryRuntimeData> || std::is_same_v<QueryDataType, FSweepQueryRuntimeData>)
			{
				if (bUpdateLength)
				{
					QueryData.SetCurrentLength(TargetLength);
				}
			}
			return Results.Num() >= MaxResults ? EVisitResult::Stop : EVisitResult::Continue;
		}

		void Reset()
		{
			Results.Reset();
		}

		int32 MaxResults = TNumericLimits<int32>::Max();
		bool bUpdateLength = false;
		FReal TargetLength = 1;
		TArray<FUserDataType> Results;
	};

	using FTestOverlapCollectorVisitor = TTestCollectorVisitor<FOverlapQueryRuntimeData, TOverlapVisitorAdapter>;
	using FTestRaycastCollectorVisitor = TTestCollectorVisitor<FRaycastQueryRuntimeData, TRaycastVisitorAdapter>;
	using FTestSweepCollectorVisitor = TTestCollectorVisitor<FSweepQueryRuntimeData, TSweepVisitorAdapter>;

	// Visitor that collects all results from visit calls. This is used to check all expected results are returned. 
	// As traversal order is not guaranteed, this sorts the results to ensure consistent and stable tests.
	// Note: The results are sorted by UserData0 then UserData1.
	struct FTestSelfQueryCollectorVisitor : public TSelfQueryVisitorAdapter<FTestSelfQueryCollectorVisitor>
	{
		struct FPair
		{
			FPair() = default;
			FPair(const FUserDataType& InUserData0, const FUserDataType& InUserData1)
			{
				// Store the pairs in a consistent ordering so we can easily diff
				if (InUserData0 < InUserData1)
				{
					UserData0 = InUserData0;
					UserData1 = InUserData1;
				}
				else
				{
					UserData0 = InUserData1;
					UserData1 = InUserData0;
				}
			}

			bool operator<(const FPair& Other) const
			{
				if (UserData0 == Other.UserData0)
				{
					return UserData1 < Other.UserData1;
				}
				return UserData0 < Other.UserData0;
			}
			bool operator==(const FPair& Other) const = default;

			FUserDataType UserData0;
			FUserDataType UserData1;
		};

		EVisitResult Visit(const FUserDataType& UserData0, const FUserDataType& UserData1)
		{
			Results.Emplace(FPair(UserData0, UserData1));
			// Insertion sort would be better to do here...
			Results.Sort();
			return Results.Num() >= MaxResults ? EVisitResult::Stop : EVisitResult::Continue;
		}

		void Reset()
		{
			Results.Reset();
		}

		int32 MaxResults = TNumericLimits<int32>::Max();
		TArray<FPair> Results;
	};

	// This provides `tostring` functionality for catch2 when the test fails.
	inline std::ostream& operator<<(std::ostream& stream, const FTestSelfQueryCollectorVisitor::FPair& value)
	{
		stream << "(" << value.UserData0 << "," << value.UserData1 << ")";
		return stream;
	}

	// Visitor that simulates typical "find first" behavior. This takes an array of objects that are assumed to have an "Aabb" field. 
	// For every hit object, the length of the query is truncated. This results in selecting the first hit object.
	// This is primarily for performance tests and checking how early traversals can finish.
	// Note: This also assumes the user data maps to the array index.
	template <typename ObjectType, typename QueryDataType, template <typename> typename AdapterType>
	struct TTestFindFirstObjectVisitor : public AdapterType<TTestFindFirstObjectVisitor<ObjectType, QueryDataType, AdapterType>>
	{
		TTestFindFirstObjectVisitor(const TArray<ObjectType>& InObjects)
			: Objects(InObjects)
		{
		}

		EVisitResult Visit(const FUserDataType& UserData, QueryDataType& QueryData)
		{
			++VisitCount;
			ensure(Objects.IsValidIndex(UserData));
			const ObjectType& Object = Objects[UserData];
			FReal Time;
			if (QueryData.Test(Object.Aabb, Time))
			{
				if (Time < QueryData.GetCurrentLength())
				{
					QueryData.SetCurrentLength(Time);
					FirstHitUserData = UserData;
				}
			}
			return EVisitResult::Continue;
		}

		void Reset()
		{
			VisitCount = 0;
			FirstHitUserData = INDEX_NONE;
		}

		const TArray<ObjectType>& Objects;
		FUserDataType FirstHitUserData = INDEX_NONE;
		// Visit count isn't too useful for CHECK tests since the initial traversal order often can't be relied on,
		// however this value can be useful when debugging to check expectations.
		int32 VisitCount = 0;
	};

	template <typename ObjectType>
	using TTestFindFirstObjectRaycastVisitor = TTestFindFirstObjectVisitor<ObjectType, FRaycastQueryRuntimeData, TRaycastVisitorAdapter>;
	template <typename ObjectType>
	using TTestFindFirstObjectSweepVisitor = TTestFindFirstObjectVisitor<ObjectType, FSweepQueryRuntimeData, TSweepVisitorAdapter>;
} // namespace Chaos::SpatialPartition
